// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AudioEditor : ModuleRules
{
	public AudioEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AudioExtensions",
				"AudioLinkEngine",
				"DeveloperSettings",
				"EditorSubsystem",
				"GameProjectGeneration",
				"ToolMenus",
				"ToolWidgets",
				"UMG",
				"UMGEditor",
			}
		);

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AudioMixer",
				"ClassViewer",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"DetailCustomizations",
				"EditorFramework",
				"Engine",
				"GraphEditor",
				"InputCore",
				"Kismet",
				"Landscape",
				"LevelEditor",
				"PropertyEditor",
				"RenderCore",
				"SignalProcessing",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		PrivateIncludePathModuleNames.AddRange
		(
			new string[]
			{
				"AssetTools",
				"WorkspaceMenuStructure",
			}
		);

		// Circular references that need to be cleaned up
		CircularlyReferencedDependentModules.AddRange
		(
			new string[]
			{
				"DetailCustomizations",
			}
		);

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			string PlatformName = "Win64";
			if (Target.Architecture == UnrealArch.Arm64)
			{
				PlatformName += "/arm64";
			}

			string LibSndFilePath = Target.UEThirdPartyBinariesDirectory + "libsndfile/";
			LibSndFilePath += PlatformName;

			PublicAdditionalLibraries.Add(LibSndFilePath + "/libsndfile-1.lib");
			PublicDelayLoadDLLs.Add("libsndfile-1.dll");
			PublicIncludePathModuleNames.Add("UELibSampleRate");

			RuntimeDependencies.Add("$(EngineDir)/Binaries/ThirdParty/libsndfile/" + PlatformName + "/libsndfile-1.dll");

			if (Target.Architecture == UnrealArch.Arm64)
			{
				// NOTE: The licencing terms of mpg123 mean that it is safer for us to link it into libsndfile as a dll than as a static lib.
				PublicDelayLoadDLLs.Add("mpg123.dll");
				RuntimeDependencies.Add("$(EngineDir)/Binaries/ThirdParty/libsndfile/" + PlatformName + "/mpg123.dll");
			}

			PublicDefinitions.Add("WITH_SNDFILE_IO=1");
			PublicDefinitions.Add("ENABLE_AUDIO_DEVICE_EDITOR_SETTINGS=1");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
		{
			string PlatformName = "Mac/";
			string LibFilename = "libsndfile.1.dylib";
			string LibFolder = "libsndfile/";
			string LibSndFilePath = Path.Combine(Target.UEThirdPartyBinariesDirectory, LibFolder, PlatformName, LibFilename);

			PublicDelayLoadDLLs.Add(LibSndFilePath);
			PublicIncludePathModuleNames.Add("UELibSampleRate");
			RuntimeDependencies.Add(LibSndFilePath);
			
			PublicDefinitions.Add("WITH_SNDFILE_IO=1");
			PublicDefinitions.Add("ENABLE_AUDIO_DEVICE_EDITOR_SETTINGS=0");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			string PlatformName = "Linux/";
			string LibFilename = "libsndfile.so.1";
			string LibFolder = "libsndfile/";
			string LibSndFilePath = Path.Combine(Target.UEThirdPartyBinariesDirectory, LibFolder, PlatformName, LibFilename);

			PublicDelayLoadDLLs.Add(LibSndFilePath);
			PublicIncludePathModuleNames.Add("UELibSampleRate");
			RuntimeDependencies.Add(LibSndFilePath);
			
			PublicDefinitions.Add("WITH_SNDFILE_IO=1");
			PublicDefinitions.Add("ENABLE_AUDIO_DEVICE_EDITOR_SETTINGS=0");
		}
		else
		{
			PublicDefinitions.Add("WITH_SNDFILE_IO=0");
			PublicDefinitions.Add("ENABLE_AUDIO_DEVICE_EDITOR_SETTINGS=0");
		}
	}
}
