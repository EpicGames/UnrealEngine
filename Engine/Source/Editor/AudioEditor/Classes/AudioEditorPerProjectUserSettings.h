// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "AudioEditorPerProjectUserSettings.generated.h"

UENUM()
enum class EUseTemplateSoundWaveDuringAssetImport : uint8
{
	AlwaysPrompt = 0,
	AlwaysUse = 1,
	NeverUse = 2
};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings, meta = (DisplayName = "Audio Editor Per Project Settings"))
class UAudioEditorPerProjectUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Whether to use the selected Sound Wave in the Content Browser as a template for sound(s) being imported. */
	UPROPERTY(EditAnywhere, config, Category = AssetImport)
	EUseTemplateSoundWaveDuringAssetImport UseTemplateSoundWaveDuringAssetImport = EUseTemplateSoundWaveDuringAssetImport::NeverUse;

	//~ Begin UDeveloperSettings
	virtual FName GetCategoryName() const override { return TEXT("General"); }
	//~ End UDeveloperSettings

};
