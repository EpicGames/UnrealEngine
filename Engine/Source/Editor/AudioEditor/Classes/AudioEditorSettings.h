// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "Engine/DeveloperSettings.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailCustomization.h"
#include "HAL/Platform.h"
#include "ToolMenuMisc.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AudioEditorSettings.generated.h"

#define UE_API AUDIOEDITOR_API

namespace Audio
{
	struct FAudioPlatformDeviceInfo;
}

class UObject;
struct FPropertyChangedEvent;

USTRUCT()
struct FAudioPlatform
{
	GENERATED_BODY()
	
	UPROPERTY(config)
	FString DisplayName;
	
	UPROPERTY(config)
	FString ModuleName;

	FAudioPlatform() = default;

	FAudioPlatform(const FString& InDisplayName, const FString& InModuleName)
		: DisplayName(InDisplayName)
		, ModuleName(InModuleName)
	{}

	bool operator<(const FAudioPlatform& Other) const
	{
		return DisplayName < Other.DisplayName;
	}
};

UCLASS(MinimalAPI, config = EditorSettings, meta = (DisplayName = "Audio"))
class UAudioEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

protected:

	/** Whether or not should Audio Attenuation be used by default, for Non-Game Worlds*/
	UPROPERTY(EditAnywhere, config, Category = NonGameWorld)
	bool bUseAudioAttenuation = true;

public:
	/** Whether to pin the Sound Cue asset type when creating new assets. Requires editor restart to take effect. */
	UPROPERTY(EditAnywhere, config, Category = AssetMenu, meta = (ConfigRestartRequired = true))
	bool bPinSoundCueInAssetMenu = true;

	/** Whether to pin the Sound Cue Template asset type when creating new assets. Requires editor restart to take effect. */
	UPROPERTY(EditAnywhere, config, Category = AssetMenu, meta = (ConfigRestartRequired = true))
	bool bPinSoundCueTemplateInAssetMenu = false;

	/** Whether to pin the Sound Attenuation asset type when creating new assets. Requires editor restart to take effect. */
	UPROPERTY(EditAnywhere, config, Category = AssetMenu, meta = (ConfigRestartRequired = true))
	bool bPinSoundAttenuationInAssetMenu = true;

	/** Whether to pin the Sound Concurrency asset type when creating new assets. Requires editor restart to take effect. */
	UPROPERTY(EditAnywhere, config, Category = AssetMenu, meta = (ConfigRestartRequired = true))
	bool bPinSoundConcurrencyInAssetMenu = true;

	/** The device id of the currently selected audio output device. Requires editor restart to take effect */
	UPROPERTY(EditAnywhere, config, Category = AudioOutputDevice, DisplayName = "Platform Audio API", meta = (ConfigRestartRequired = true))
	FString AudioMixerModuleName;

	/** Use current audio playback device selected in the operating system. */
	UPROPERTY(EditAnywhere, config, Category = AudioOutputDevice, DisplayName = "Use System Default Audio Device")
	bool bUseSystemDevice = true;
	
	/** The device id of the currently selected audio output device. */
	UPROPERTY(EditAnywhere, config, Category = AudioOutputDevice)
	FString AudioOutputDeviceId;

	/** Array of available audio platforms. */
	UPROPERTY(config)
	TArray<FAudioPlatform> AudioMixerPlatforms;

	/** Use the submix tree when playing preview sounds */
	UPROPERTY(EditAnywhere, config, Category = EditorPlaybackOptions, DisplayName = "Use Submixes When Previewing Sounds")
	bool bUseSubmixesForPreviewSound = true;

	/** Specify placement of sound menus when right-clicking on sound asset types in Content Browser (ex. 'Last' will sort to bottom, 'First' to top). */
	UPROPERTY(EditAnywhere, config, Category = AssetActionMenu, meta = (GetOptions = "AudioEditor.AudioEditorSettings.GetAllowedMenuPositions"))
	FName MenuPosition = "Last";

	/** Set and apply, whether audio attenuation is used for non-game worlds*/
	UE_API void SetUseAudioAttenuation(bool bInUseAudioAttenuation);

	/** Is audio attenuation used for non-game worlds*/
	bool IsUsingAudioAttenuation() const { return bUseAudioAttenuation; };

	//~ Begin UDeveloperSettings
	virtual FName GetCategoryName() const override { return TEXT("General"); }
	//~ End UDeveloperSettings

	//~ Begin UObject
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostInitProperties() override;
	//~ End UObject

	UE_API void RebuildSoundContextMenuSections() const;

	UFUNCTION()
	static TArray<FName> GetAllowedMenuPositions();

private:

	/** Apply non-game world attenuation setting for all AudioDevices.*/
	UE_API void ApplyAttenuationForAllAudioDevices();

	/** Apply non-game world attenuation setting for AudioDevice with given ID.*/
	UE_API void ApplyAttenuationForAudioDevice(Audio::FDeviceId InDeviceID);
};

class FAudioOutputDeviceCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	FAudioOutputDeviceCustomization();
	virtual ~FAudioOutputDeviceCustomization() override;
	
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization

	static bool IsDeviceSettingsEnabled();

private:
	TSharedPtr<IPropertyHandle> AudioPlatformPropertyHandle;
	TSharedPtr<IPropertyHandle> UseSystemDevicePropertyHandle;
	TSharedPtr<IPropertyHandle> AudioDeviceIdPropertyHandle;
	TSharedPtr<SWidget> DeviceListComboButton;
	FString CurrentDeviceName;
	TWeakPtr<FActiveTimerHandle> DeviceMenuActiveTimer;
	FTickerDelegate TickDelegate;
	FTSTicker::FDelegateHandle TickDelegateHandle;
	TArray<FAudioPlatform> AudioMixerPlatforms;
	TArray<TSharedPtr<FString>> AudioBackends;
	TSharedPtr<FString> CurrentBackendName;

	static TArray<Audio::FAudioPlatformDeviceInfo> GetAvailableAudioOutputDevices();
	static Audio::FAudioPlatformDeviceInfo GetDeviceInfo(const FString& InDeviceId);
	static TOptional<Audio::FAudioPlatformDeviceInfo> FindDefaultOutputDevice();
	static FString GetDeviceNameForDeviceId(const FString& InDeviceId);
	static FString GetCurrentAudioMixerDeviceName();
	static void SetIsListeningForDeviceEvents(bool bInListeningForDeviceEvents);
	static void RequestDeviceSwap(const FString& InDeviceId);
	static bool IsAggregateHardwareDeviceId(const FString& InDeviceId);
	
	void AddPlatformRow(IDetailCategoryBuilder& Category, const IDetailLayoutBuilder& DetailLayout);
	TSharedRef<SWidget> OnGenerateDeviceMenu();
	void MenuItemDeviceSelected(Audio::FAudioPlatformDeviceInfo InDeviceInfo);
	bool Tick(float DeltaTime);
	TSharedPtr<FString> GetCurrentBackendName() const;
	void SetCurrentBackendName(const FString& InBackendName);
};

#undef UE_API
