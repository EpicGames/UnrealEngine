// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// DialogueWaveFactory
//~=============================================================================

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "DialogueWaveFactory.generated.h"

class FFeedbackContext;
class UClass;
class UDialogueVoice;
class UObject;

UCLASS(hidecategories=Object, MinimalAPI)
class UDialogueWaveFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	/** An initial sound wave to place in the newly created dialogue wave */
	UPROPERTY()
	TObjectPtr<class USoundWave> InitialSoundWave;

	/** An initial speaking dialogue voice to place in the newly created dialogue wave */
	UPROPERTY()
	TObjectPtr<class UDialogueVoice> InitialSpeakerVoice;

	/** Whether an initial target dialogue voice should be set */
	UPROPERTY()
	bool HasSetInitialTargetVoice;

	/** An initial target dialogue voices to place in the newly created dialogue wave */
	UPROPERTY()
	TArray<TObjectPtr<UDialogueVoice>> InitialTargetVoices;
};



