// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReimportSoundFactory
//~=============================================================================

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EditorReimportHandler.h"
#include "Factories/SoundFactory.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ReimportSoundFactory.generated.h"

class FString;
class UObject;

UCLASS(hidecategories=Object, collapsecategories)
class UReimportSoundFactory : public USoundFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

	//~ Being UFactory Interface
	virtual void CleanUp() override;
	//~ End UFactory Interface

	int8 OverwriteOtherAssetTypes;
};
