// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReverbEffectFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ReverbEffectFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(MinimalAPI, hidecategories=Object)
class UReverbEffectFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
