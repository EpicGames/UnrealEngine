// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SoundCueFactoryNew
//~=============================================================================

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "SoundCueFactoryNew.generated.h"

class FFeedbackContext;
class UClass;
class UDialogueWave;
class UObject;
class USoundCue;
class USoundNode;
class USoundNodeRandom;
class USoundWave;

UCLASS(hidecategories=Object, MinimalAPI)
class USoundCueFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	/** Initial sound wave to place in the newly created cue */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Array InitialSoundWaves instead."))
	TObjectPtr<USoundWave> InitialSoundWave;

	/** Initial sound wave(s) to place in the newly created cue(s) */
	UPROPERTY(EditAnywhere, Category = Audio)
	TArray<TWeakObjectPtr<USoundWave>> InitialSoundWaves;

	/** An initial dialogue wave to place in the newly created cue */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Array InitialDialogueWaves instead."))
	TObjectPtr<UDialogueWave> InitialDialogueWave;

	/** Initial dialogue wave(s) to place in the newly created cue(s) */
	UPROPERTY(EditAnywhere, Category = Audio)
	TArray<TWeakObjectPtr<UDialogueWave>> InitialDialogueWaves;

protected:

	static USoundNodeRandom* InsertRandomNode(USoundCue* SoundCue, int32 NodePosX, int32 NodePosY);
	static USoundNode* CreateSoundPlayerNode(USoundCue* SoundCue, UObject* SoundObject, int32 NodePosX, int32 NodePosY);
};



