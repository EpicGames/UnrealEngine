// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SoundFactory
//~=============================================================================

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "SoundFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;
class USoundWave;

UCLASS(MinimalAPI, hidecategories=Object)
class USoundFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** If enabled, a sound cue will automatically be created for the sound */
	UPROPERTY(EditAnywhere, Category=SoundFactory, meta=(ToolTip="If enabled, a sound cue will automatically be created for the sound"))
	uint32 bAutoCreateCue:1;

	/** If enabled, the created sound cue will include a attenuation node */
	UPROPERTY(EditAnywhere, Category=SoundFactory, meta=(ToolTip="If enabled, the created sound cue will include a attenuation node"))
	uint32 bIncludeAttenuationNode:1;

	/** If enabled, the created sound cue will include a looping node */
	UPROPERTY(EditAnywhere, Category=SoundFactory, meta=(ToolTip="If enabled, the created sound cue will include a looping node"))
	uint32 bIncludeLoopingNode:1;

	/** If enabled, the created sound cue will include a modulator node */
	UPROPERTY(EditAnywhere, Category=SoundFactory, meta=(ToolTip="If enabled, the created sound cue will include a modulator node"))
	uint32 bIncludeModulatorNode:1;

	/** The volume of the created sound cue */
	UPROPERTY(EditAnywhere, Category=SoundFactory, meta=(ToolTip="The volume of the created sound cue"))
	float CueVolume;

	/** If not empty, imported waves will be placed in PackageCuePackageSuffix, but only if bAutoCreateCue is true. */
	UPROPERTY(EditAnywhere, Category=SoundFactory, meta=(ToolTip="If not empty, generated SoundCues will be placed in PackageCuePackageSuffix, but only if bAutoCreateCue is true"))
	FString CuePackageSuffix;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateBinary( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* FileType, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn ) override;
	//~ End UFactory Interface

	uint8 SuppressImportDialogOptions;

public:
	/** Suppresses all dialogs pertaining to importing using factory */
	AUDIOEDITOR_API void SuppressImportDialogs();

	virtual void CleanUp() override;

protected:
	enum ESuppressImportDialog
	{
		None = 0,
		Overwrite = 1 << 0,
		UseTemplate = 1 << 1
	};

private:
	void UpdateTemplate();

	TWeakObjectPtr<USoundWave> TemplateSoundWave;


	UObject* CreateObject(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* FileType, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn);
};
