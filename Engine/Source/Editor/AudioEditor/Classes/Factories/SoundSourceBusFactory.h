// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SoundSubmixFactory
//~=============================================================================

#pragma once
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundSourceBusFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(MinimalAPI, hidecategories=Object)
class USoundSourceBusFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	//~ Begin UFactory Interface	
};
