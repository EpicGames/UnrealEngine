// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Sound/SoundEffectSubmix.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundSubmixEffectFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;
class USoundEffectSubmixPreset;

UCLASS(MinimalAPI, hidecategories=Object)
class USoundSubmixEffectFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** The type of sound source effect preset that will be created */
	UPROPERTY(EditAnywhere, Category = CurveFactory)
	TSubclassOf<USoundEffectSubmixPreset> SoundEffectSubmixPresetClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	//~ Begin UFactory Interface	
};


