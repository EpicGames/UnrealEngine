// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SoundSubmixFactory
//~=============================================================================

#pragma once
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundSubmixFactory.generated.h"

#define UE_API AUDIOEDITOR_API

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(MinimalAPI, hidecategories=Object)
class USoundSubmixFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	UE_API virtual bool CanCreateNew() const override;
	//~ Begin UFactory Interface	
};

UCLASS(MinimalAPI, hidecategories = Object)
class USoundfieldSubmixFactory: public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual bool CanCreateNew() const override;
	//~ Begin UFactory Interface	
};

UCLASS(MinimalAPI, hidecategories = Object)
class UEndpointSubmixFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual bool CanCreateNew() const override;
	//~ Begin UFactory Interface	
};

UCLASS(MinimalAPI, hidecategories = Object)
class USoundfieldEndpointSubmixFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual bool CanCreateNew() const override;
	//~ Begin UFactory Interface	
};

#undef UE_API
