// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundClassGraph.generated.h"

class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class USoundClassGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	/**
	 * Set the SoundClass which forms the root of this graph
	 */
	AUDIOEDITOR_API void SetRootSoundClass(class USoundClass* InSoundClass);
	/**
	 * Get the SoundClass which forms the root of this graph
	 */
	class USoundClass* GetRootSoundClass() const;
	/**
	 * Completely rebuild the graph from the root, removing all old nodes
	 */
	AUDIOEDITOR_API void RebuildGraph();
	/**
	 * Display SoundClasses (and all of their children) that have been dragged onto the editor
	 *
	 * @param	SoundClasses	SoundClasses not already represented on the graph
	 * @param	NodePosX		X coordinate classes were dropped at
	 * @param	NodePosY		Y coordinate classes were dropped at
	 */
	void AddDroppedSoundClasses(const TArray<class USoundClass*>& SoundClasses, int32 NodePosX, int32 NodePosY);

	/**
	 * Display a new SoundClass that has just been created using the editor
	 *
	 * @param	FromPin		The Pin that was dragged from to create the SoundClass (may be NULL)
	 * @param	SoundClass	The newly created SoundClass
	 * @param	NodePosX	X coordinate new class was created at
	 * @param	NodePosY	Y coordinate new class was created at
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 */
	AUDIOEDITOR_API void AddNewSoundClass(UEdGraphPin* FromPin, class USoundClass* SoundClass, int32 NodePosX, int32 NodePosY, bool bSelectNewNode = true);
	/**
	 * Checks whether a SoundClass is already represented on this graph
	 */
	bool IsClassDisplayed(class USoundClass* SoundClass) const;
	/**
	 * Use this graph to re-link all of the SoundClasses it represents after a change in linkage
	 */
	void LinkSoundClasses();
	/**
	 * Re-link all of the nodes in this graph after a change to SoundClass linkage
	 */
	AUDIOEDITOR_API void RefreshGraphLinks();
	/**
	 * Recursively remove a set of nodes from this graph and re-link SoundClasses afterwards
	 */
	AUDIOEDITOR_API void RecursivelyRemoveNodes(const TSet<class UObject*> NodesToRemove);

private:
	/**
	 * Construct Nodes to represent a SoundClass and all of its children
	 *
	 * @param	SoundClass	The SoundClass to represent
	 * @param	NodePosX	X coordinate to place first node at
	 * @param	NodePosY	Y coordinate to place first node at
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 * @return	Total height of all constructed nodes (used to arrange multiple new nodes)
	 */
	int32 ConstructNodes(class USoundClass* SoundClass, int32 NodePosX, int32 NodePosY, bool bSelectNewNode = true);
	/**
	 * Recursively build a map of child counts for each SoundClass to arrange them correctly
	 *
	 * @param	ParentClass		The class we are getting child counts for
	 * @param	OutChildCounts	Map of child counts
	 * @return	Total child count for ParentClass
	 */
	int32 RecursivelyGatherChildCounts(class USoundClass* ParentClass, TMap<class USoundClass*, int32>& OutChildCounts);
	/**
	 * Recursively Construct Nodes to represent the children of a SoundClass
	 *
	 * @param	ParentNode		The Node we are constructing children for
	 * @param	InChildCounts	Map of child counts
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 * @return	Total height of constructed nodes (used to arrange next new node)
	 */
	int32 RecursivelyConstructChildNodes(class USoundClassGraphNode* ParentNode, const TMap<class USoundClass*, int32>& InChildCounts, bool bSelectNewNode = true);
	/**
	 * Recursively remove a node and its children from the graph
	 */
	void RecursivelyRemoveNode(class USoundClassGraphNode* ParentNode);
	/**
	 * Remove all Nodes from the graph
	 */
	void RemoveAllNodes();
	/**
	 * Create a new node to represent a SoundClass
	 *
	 * @param	SoundClass	The SoundClass to represent
	 * @param	NodePosX	X coordinate to place node at
	 * @param	NodePosY	Y coordinate to place node at
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 * @return	Either a new node or an existing node representing the class
	 */
	class USoundClassGraphNode* CreateNode(class USoundClass* SoundClass, int32 NodePosX, int32 NodePosY, bool bSelectNewNode = true);
	/**
	 * Find an existing node that represents a given SoundClass
	 */
	class USoundClassGraphNode* FindExistingNode(class USoundClass* SoundClass) const;

private:
	/** SoundClass which forms the root of this graph */
	class USoundClass*	RootSoundClass;
};

