// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "SoundClassGraphNode.generated.h"

class UEdGraphPin;
class UEdGraphSchema;
class UObject;
class USoundClass;

UCLASS(MinimalAPI)
class USoundClassGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** The SoundNode this represents */
	UPROPERTY(VisibleAnywhere, instanced, Category=Sound)
	TObjectPtr<USoundClass>		SoundClass;

	/** Get the Pin that connects to all children */
	UEdGraphPin* GetChildPin() const { return ChildPin; }
	/** Get the Pin that connects to its parent */
	UEdGraphPin* GetParentPin() const { return ParentPin; }
	/** Check whether the children of this node match the SoundClass it is representing */
	bool CheckRepresentsSoundClass();

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void AllocateDefaultPins() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanUserDeleteNode() const override;
	//~ End UEdGraphNode Interface.

private:
	/** Pin that connects to all children */
	UEdGraphPin* ChildPin;
	/** Pin that connects to its parent */
	UEdGraphPin* ParentPin;
};
