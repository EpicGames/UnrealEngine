// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "SoundClassGraphSchema.generated.h"

#define UE_API AUDIOEDITOR_API

class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;

/** Action to add a node to the graph */
USTRUCT()
struct FSoundClassGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FSoundClassGraphSchemaAction_NewNode"); return Type;}

	FSoundClassGraphSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, NewSoundClassName(TEXT("ClassName"))
	{}

	FSoundClassGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NewSoundClassName(TEXT("ClassName"))
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	/** Name for the new SoundClass */
	FString NewSoundClassName;
};

UCLASS(MinimalAPI)
class USoundClassGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	/** Check whether connecting these pins would cause a loop */
	bool ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	//~ Begin EdGraphSchema Interface
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const override;
	virtual bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	//~ End EdGraphSchema Interface
};

#undef UE_API
