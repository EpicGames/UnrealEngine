// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundCueGraph.generated.h"

class UObject;

UCLASS(MinimalAPI)
class USoundCueGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	/** Returns the SoundCue that contains this graph */
	AUDIOEDITOR_API class USoundCue* GetSoundCue() const;
};

