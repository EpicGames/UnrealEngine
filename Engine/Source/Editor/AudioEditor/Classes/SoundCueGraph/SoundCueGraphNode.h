// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "SoundCueGraph/SoundCueGraphNode_Base.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "SoundCueGraphNode.generated.h"

class UEdGraphPin;
class UObject;
class USoundNode;

UCLASS(MinimalAPI)
class USoundCueGraphNode : public USoundCueGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	/** The SoundNode this represents */
	UPROPERTY(VisibleAnywhere, instanced, Category=Sound)
	TObjectPtr<USoundNode> SoundNode;

	/** Set the SoundNode this represents (also assigns this to the SoundNode in Editor)*/
	AUDIOEDITOR_API void SetSoundNode(USoundNode* InSoundNode);
	/** Fix up the node's owner after being copied */
	AUDIOEDITOR_API void PostCopyNode();
	/** Create a new input pin for this node */
	AUDIOEDITOR_API void CreateInputPin();
	/** Add an input pin to this node and recompile the SoundCue */
	AUDIOEDITOR_API void AddInputPin();
	/** Remove a specific input pin from this node and recompile the SoundCue */
	AUDIOEDITOR_API void RemoveInputPin(UEdGraphPin* InGraphPin);
	/** Estimate the width of this Node from the length of its title */
	AUDIOEDITOR_API int32 EstimateNodeWidth() const;
	/** Checks whether an input can be added to this node */
	AUDIOEDITOR_API bool CanAddInputPin() const;


	// USoundCueGraphNode_Base interface
	virtual void CreateInputPins() override;
	// End of USoundCueGraphNode_Base interface

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PrepareForCopying() override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDocumentationExcerptName() const override;
	// End of UEdGraphNode interface

	// UObject interface
	virtual void PostLoad() override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	// End of UObject interface

private:
	/** Make sure the soundnode is owned by the SoundCue */
	void ResetSoundNodeOwner();
};
