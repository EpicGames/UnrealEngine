// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundCueGraphNode_Base.generated.h"

class UEdGraphPin;
class UEdGraphSchema;
class UObject;

UCLASS(MinimalAPI)
class USoundCueGraphNode_Base : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** Create all of the input pins required */
	virtual void CreateInputPins() {};
	/** Is this the undeletable root node */
	virtual bool IsRootNode() const {return false;}

	/** Get the Output Pin (should only ever be one) */
	AUDIOEDITOR_API class UEdGraphPin* GetOutputPin();
	/** Get all of the Input Pins */
	AUDIOEDITOR_API void GetInputPins(TArray<class UEdGraphPin*>& OutInputPins);
	/** Get a single Input Pin via its index */
	AUDIOEDITOR_API class UEdGraphPin* GetInputPin(int32 InputIndex);
	/** Get the current Input Pin count */
	AUDIOEDITOR_API int32 GetInputCount() const;

	/**
	 * Handles inserting the node between the FromPin and what the FromPin was original connected to
	 *
	 * @param FromPin			The pin this node is being spawned from
	 * @param NewLinkPin		The new pin the FromPin will connect to
	 * @param OutNodeList		Any nodes that are modified will get added to this list for notification purposes
	 */
	void InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList);

	// UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual void ReconstructNode() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual FString GetDocumentationLink() const override;
	// End of UEdGraphNode interface.
};
