// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SoundCueGraph/SoundCueGraphNode_Base.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundCueGraphNode_Root.generated.h"

class UObject;

UCLASS(MinimalAPI)
class USoundCueGraphNode_Root : public USoundCueGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UEdGraphNode interface

	// USoundCueGraphNode_Base interface
	virtual void CreateInputPins() override;
	virtual bool IsRootNode() const override {return true;}
	// End of USoundCueGraphNode_Base interface
};
