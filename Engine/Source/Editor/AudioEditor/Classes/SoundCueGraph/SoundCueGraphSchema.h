// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "ClassViewerModule.h"
#include "EdGraph/EdGraphSchema.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SSoundCuePalette.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"

#include "SoundCueGraphSchema.generated.h"

#define UE_API AUDIOEDITOR_API

class FString;
class IAssetReferenceFilter;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
class USoundNode;
struct FAssetData;
struct FEdGraphPinType;

/** Action to add a node to the graph */
USTRUCT()
struct FSoundCueGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Class of node we want to create */
	UPROPERTY()
	TSubclassOf<USoundNode> SoundNodeClass;


	FSoundCueGraphSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, SoundNodeClass(NULL)
	{}

	FSoundCueGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, SoundNodeClass(NULL)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

private:
	/** Connects new node to output of selected nodes */
	UE_API void ConnectToSelectedNodes(USoundNode* NewNodeclass, UEdGraph* ParentGraph) const;
};

/** Action to add nodes to the graph based on selected objects*/
USTRUCT()
struct FSoundCueGraphSchemaAction_NewFromSelected : public FSoundCueGraphSchemaAction_NewNode
{
	GENERATED_USTRUCT_BODY();

	FSoundCueGraphSchemaAction_NewFromSelected() 
		: FSoundCueGraphSchemaAction_NewNode()
	{}

	FSoundCueGraphSchemaAction_NewFromSelected(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FSoundCueGraphSchemaAction_NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping) 
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to create new comment */
USTRUCT()
struct FSoundCueGraphSchemaAction_NewComment : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	FSoundCueGraphSchemaAction_NewComment() 
		: FEdGraphSchemaAction()
	{}

	FSoundCueGraphSchemaAction_NewComment(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to paste clipboard contents into the graph */
USTRUCT()
struct FSoundCueGraphSchemaAction_Paste : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	FSoundCueGraphSchemaAction_Paste() 
		: FEdGraphSchemaAction()
	{}

	FSoundCueGraphSchemaAction_Paste(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

UCLASS(MinimalAPI)
class USoundCueGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	/** Check whether connecting these pins would cause a loop */
	bool ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	/** Helper method to add items valid to the palette list */
	AUDIOEDITOR_API void GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder) const;

	/** Attempts to connect the output of multiple nodes to the inputs of a single one */
	void TryConnectNodes(const TArray<USoundNode*>& OutputNodes, USoundNode* InputNode) const;

	//~ Begin EdGraphSchema Interface
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const override;
	virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	//~ End EdGraphSchema Interface
	
	/** Update list of component classes */
	void UpdateSoundNodeList(const SSoundCuePalette::FSoundNodeFilterData&);

private:
	/** Adds actions for creating every type of SoundNode */
	void GetAllSoundNodeActions(FGraphActionMenuBuilder& ActionMenuBuilder, bool bShowSelectedActions) const;
	/** Adds action for creating a comment */
	void GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph = NULL) const;

	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
	
	/** Soundnodes that can be selected in this graph */
	TArray<TSubclassOf<USoundNode>> AllowedSoundNodes;
};

#undef UE_API
