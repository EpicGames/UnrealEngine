// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "SoundSubmixGraph.generated.h"

class UEdGraphPin;
class UObject;
// Forward Declarations
class USoundSubmixBase;
class USoundSubmixGraphNode;


UCLASS(MinimalAPI)
class USoundSubmixGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Set the SoundSubmix which forms the root of this graph
	 */
	AUDIOEDITOR_API void SetRootSoundSubmix(USoundSubmixBase* InSoundSubmix);

	/**
	 * Get the SoundSubmix which forms the root of this graph
	 */
	USoundSubmixBase* GetRootSoundSubmix() const;

	/**
	 * Completely rebuild the graph from the root, removing all old nodes
	 */
	AUDIOEDITOR_API void RebuildGraph();

	/**
	 * Display SoundSubmixes (and all of their children) that have been dragged onto the editor
	 *
	 * @param	SoundSubmixes	SoundSubmixes not already represented on the graph
	 * @param	NodePosX		X coordinate submix(es) were dropped at
	 * @param	NodePosY		Y coordinate submix(es) were dropped at
	 */
	void AddDroppedSoundSubmixes(const TSet<USoundSubmixBase*>& SoundSubmixes, int32 NodePosX, int32 NodePosY);

	/**
	 * Display a new SoundSubmix that has just been created using the editor
	 *
	 * @param	FromPin		The Pin that was dragged from to create the SoundSubmix (may be NULL)
	 * @param	SoundClass	The newly created SoundSubmix
	 * @param	NodePosX	X coordinate new submix was created at
	 * @param	NodePosY	Y coordinate new submix was created at
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 */
	AUDIOEDITOR_API void AddNewSoundSubmix(UEdGraphPin* FromPin, USoundSubmixBase* SoundSubmix, int32 NodePosX, int32 NodePosY, bool bSelectNewNode = true);

	/**
	 * Checks whether a SoundSubmix is already represented on this graph
	 */
	bool IsSubmixDisplayed(USoundSubmixBase* SoundSubmix) const;

	/**
	 * Use this graph to re-link all of the SoundSubmixes it represents after a change in linkage
	 */
	void LinkSoundSubmixes();

	/**
	 * Re-link all of the nodes in this graph after a change to SoundSubmix linkage
	 */
	AUDIOEDITOR_API void RefreshGraphLinks();

	/**
	 * Recursively remove a set of nodes from this graph and re-link SoundSubmixes afterwards
	 */
	AUDIOEDITOR_API void RecursivelyRemoveNodes(const TSet<UObject*> NodesToRemove);

	/**
	 * Find an existing node that represents a given SoundSubmix
	 */
	AUDIOEDITOR_API USoundSubmixGraphNode* FindExistingNode(USoundSubmixBase* SoundSubmix) const;

private:
	/**
	 * Construct Nodes to represent a SoundSubmix and all of its children
	 *
	 * @param	SoundSubmix	The SoundSubmix to represent
	 * @param	NodePosX	X coordinate to place first node at
	 * @param	NodePosY	Y coordinate to place first node at
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 * @return	Total height of all constructed nodes (used to arrange multiple new nodes)
	 */
	int32 ConstructNodes(USoundSubmixBase* SoundSubmix, int32 NodePosX, int32 NodePosY, bool bSelectNewNode = true);
	/**
	 * Recursively build a map of child counts for each SoundSubmix to arrange them correctly
	 *
	 * @param	ParentClass		The class we are getting child counts for
	 * @param	OutChildCounts	Map of child counts
	 * @return	Total child count for ParentClass
	 */
	int32 RecursivelyGatherChildCounts(USoundSubmixBase* ParentSubmix, TMap<USoundSubmixBase*, int32>& OutChildCounts);
	/**
	 * Recursively Construct Nodes to represent the children of a SoundSubmix
	 *
	 * @param	ParentNode		The Node we are constructing children for
	 * @param	InChildCounts	Map of child counts
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 * @return	Total height of constructed nodes (used to arrange next new node)
	 */
	int32 RecursivelyConstructChildNodes(USoundSubmixGraphNode* ParentNode, const TMap<USoundSubmixBase*, int32>& InChildCounts, bool bSelectNewNode = true);
	/**
	 * Recursively remove a node and its children from the graph
	 */
	void RecursivelyRemoveNode(USoundSubmixGraphNode* ParentNode);
	/**
	 * Remove all Nodes from the graph
	 */
	void RemoveAllNodes();
	/**
	 * Create a new node to represent a SoundSubmix
	 *
	 * @param	SoundSubmix	The SoundSubmix to represent
	 * @param	NodePosX	X coordinate to place node at
	 * @param	NodePosY	Y coordinate to place node at
	 * @param bSelectNewNode	Whether or not to select the new node being created
	 * @return	Either a new node or an existing node representing the class
	 */
	USoundSubmixGraphNode* CreateNode(USoundSubmixBase* SoundSubmix, int32 NodePosX, int32 NodePosY, bool bSelectNewNode = true);

private:
	/** SoundSubmix which forms the root of this graph */
	UPROPERTY(Transient)
	TObjectPtr<USoundSubmixBase> RootSoundSubmix = nullptr;

	UPROPERTY(Transient)
	TArray<TObjectPtr<USoundSubmixBase>> StaleRoots;
};

