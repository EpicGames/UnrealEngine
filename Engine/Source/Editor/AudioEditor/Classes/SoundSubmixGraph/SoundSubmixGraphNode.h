// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio/AudioWidgetSubsystem.h"
#include "AudioDevice.h"
#include "Blueprint/UserWidget.h"
#include "CoreMinimal.h"
#include "DSP/EnvelopeFollower.h"
#include "DSP/MultithreadedPatching.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SGraphNode.h"
#include "Sound/SoundSubmix.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include "SoundSubmixGraphNode.generated.h"

class SWidget;
class UEdGraphPin;
class UEdGraphSchema;
class UObject;
// Forward Declarations
class USoundSubmixBase;
class UUserWidget;




class SSubmixGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SSubmixGraphNode) {}

	SLATE_ARGUMENT(TWeakObjectPtr<USoundSubmixBase>, SubmixBase)
	SLATE_ARGUMENT(TWeakObjectPtr<UUserWidget>, SubmixNodeUserWidget)

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	TSharedRef<SWidget> CreateNodeContentArea() override;

private:
	TWeakObjectPtr<USoundSubmixBase> SubmixBase;
	TWeakObjectPtr<UUserWidget> SubmixNodeUserWidget;
};

UCLASS(MinimalAPI)
class USoundSubmixGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** The SoundSubmix this represents */
	UPROPERTY(VisibleAnywhere, instanced, Category=Sound)
	TObjectPtr<USoundSubmixBase> SoundSubmix;

	/** A user widget to use to represent the graph node */
	UPROPERTY(Transient)
	TObjectPtr<UUserWidget> SubmixNodeUserWidget;

	/** Get the Pin that connects to all children */
	UEdGraphPin* GetChildPin() const { return ChildPin; }

	/** Get the Pin that connects to its parent */
	UEdGraphPin* GetParentPin() const { return ParentPin; }
	
	/** Check whether the children of this node match the SoundSubmix it is representing */
	bool CheckRepresentsSoundSubmix();

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void AllocateDefaultPins() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual bool CanUserDeleteNode() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget();
	//~ End UEdGraphNode Interface.

private:
	/** Pin that connects to all children */
	UEdGraphPin* ChildPin;
	
	/** Pin that connects to its parent */
	UEdGraphPin* ParentPin;
};
