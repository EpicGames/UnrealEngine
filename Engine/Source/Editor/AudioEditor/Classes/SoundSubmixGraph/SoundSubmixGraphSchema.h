// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "SoundSubmixGraphSchema.generated.h"

#define UE_API AUDIOEDITOR_API

class FMenuBuilder;
class FSlateRect;
class FSlateWindowElementList;
// Forward Declarations
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema;
class UObject;
struct FAssetData;
struct FEdGraphPinType;

struct FSoundSubmixGraphConnectionDrawingPolicyFactory : public FGraphPanelPinConnectionFactory
{
public:
	virtual ~FSoundSubmixGraphConnectionDrawingPolicyFactory() = default;

	// FGraphPanelPinConnectionFactory
	virtual class FConnectionDrawingPolicy* CreateConnectionPolicy(
		const UEdGraphSchema* Schema,
		int32 InBackLayerID,
		int32 InFrontLayerID,
		float ZoomFactor,
		const FSlateRect& InClippingRect,
		FSlateWindowElementList& InDrawElements,
		UEdGraph* InGraphObj) const override;
	// ~FGraphPanelPinConnectionFactory
};


// This class draws the connections for an UEdGraph using a SoundCue schema
class FSoundSubmixGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	// Times for one execution pair within the current graph
	struct FTimePair
	{
		double PredExecTime;
		double ThisExecTime;

		FTimePair()
			: PredExecTime(0.0)
			, ThisExecTime(0.0)
		{
		}
	};

	// Map of pairings
	typedef TMap<UEdGraphNode*, FTimePair> FExecPairingMap;

	// Map of nodes that preceded before a given node in the execution sequence (one entry for each pairing)
	TMap<UEdGraphNode*, FExecPairingMap> PredecessorNodes;

	UEdGraph* GraphObj;

	float ActiveWireThickness;
	float InactiveWireThickness;

public:
	FSoundSubmixGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	// FConnectionDrawingPolicy interface
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	// End of FConnectionDrawingPolicy interface
};


/** Action to add a node to the graph */
USTRUCT()
struct FSoundSubmixGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FSoundSubmixGraphSchemaAction_NewNode"); return Type;}

	FSoundSubmixGraphSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, NewSoundSubmixName(TEXT("SubmixName"))
	{}

	FSoundSubmixGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NewSoundSubmixName(TEXT("SubmixName"))
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	/** Name for the new SoundSubmix */
	FString NewSoundSubmixName;
};

UCLASS(MinimalAPI)
class USoundSubmixGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	/** Check whether connecting these pins would cause a loop */
	bool ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	//~ Begin EdGraphSchema Interface
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const override;
	virtual bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	//~ End EdGraphSchema Interface
};

#undef UE_API
