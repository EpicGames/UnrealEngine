// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"

#include "SDetailsDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinitionDefault_AudioDiffable)

EAssetCommandResult UAssetDefinitionDefault_AudioDiffable::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(DiffArgs.OldAsset, DiffArgs.NewAsset, DiffArgs.OldRevision, DiffArgs.NewRevision, GetAssetClass().Get());
	// allow users to edit NewAsset if it's a local asset
	if (DiffArgs.NewAsset != nullptr && !FPackageName::IsTempPackage(DiffArgs.NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(DiffArgs.NewAsset);
	}
	return EAssetCommandResult::Handled;
}
