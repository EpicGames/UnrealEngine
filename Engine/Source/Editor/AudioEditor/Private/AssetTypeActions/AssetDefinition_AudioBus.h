// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "CoreMinimal.h"
#include "Sound/AudioBus.h"

#include "AssetDefinition_AudioBus.generated.h"

UCLASS()
class UAssetDefinition_AudioBus : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AudioBus", "Audio Bus"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(97, 97, 85)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioBus::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio, 
					NSLOCTEXT("AssetTypeActions", "AssetAudioBusMenu", "Advanced"), 
					FCategoryPath(NSLOCTEXT("AssetTypeActions", "AssetAudioBusSubMenuSection", "Routing"), ECategoryMenuType::Section))
			};
		return Categories;
	}
	// UAssetDefinition End
};
