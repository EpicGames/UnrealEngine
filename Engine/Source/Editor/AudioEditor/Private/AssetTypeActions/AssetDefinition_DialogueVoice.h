// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "CoreMinimal.h"
#include "Sound/DialogueVoice.h"

#include "AssetDefinition_DialogueVoice.generated.h"

UCLASS()
class UAssetDefinition_DialogueVoice : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DialogueVoice", "Dialogue Voice"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(97, 85, 212)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDialogueVoice::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio,
					NSLOCTEXT("AssetDefinition", "AssetDefinition_DialogueVoiceSubMenu", "Advanced"),
					FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_DialogueVoiceSubMenuSection", "Dialogue"), ECategoryMenuType::Section))
			};

		return Categories;
	}
	// UAssetDefinition End
};
