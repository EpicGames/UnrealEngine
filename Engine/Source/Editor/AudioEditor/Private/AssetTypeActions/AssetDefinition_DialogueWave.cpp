// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DialogueWave.h"
#include "AnimationEditorUtils.h"
#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Factories/SoundCueFactoryNew.h"
#include "Sound/SoundCue.h"
#include "Toolkits/SimpleAssetEditor.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_DialogueWave"

EAssetCommandResult UAssetDefinition_DialogueWave::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UDialogueWave* DialogueWave : OpenArgs.LoadObjects<UDialogueWave>())
	{
		FSimpleAssetEditor::CreateEditor(Mode, Mode == EToolkitMode::WorldCentric ? OpenArgs.ToolkitHost : TSharedPtr<IToolkitHost>(), DialogueWave);
	}
	return EAssetCommandResult::Handled;
}

namespace MenuExtension::DialogueWave
{
	static bool CanExecutePlayCommand(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return false;
		}

		TArray<FAssetData> AssetData = CBContext->SelectedAssets;
		if (AssetData.Num() != 1)
		{
			return false;
		}

		USoundBase* Sound = nullptr;
		UDialogueWave* DialogueWave = CBContext->LoadFirstSelectedObject<UDialogueWave>();
		
		if (DialogueWave == nullptr)
		{
			return false;
		}

		for (int32 i = 0; i < DialogueWave->ContextMappings.Num(); ++i)
		{
			const FDialogueContextMapping& ContextMapping = DialogueWave->ContextMappings[i];

			Sound = DialogueWave->GetWaveFromContext(ContextMapping.Context);
			if (Sound != nullptr)
			{
				break;
			}
		}

		return Sound != nullptr;
	}

	void StopSound()
	{
		GEditor->ResetPreviewAudioComponent();
	}

	static void PlaySound(UDialogueWave* DialogueWave)
	{
		USoundBase* Sound = nullptr;

		for (int32 i = 0; i < DialogueWave->ContextMappings.Num(); ++i)
		{
			const FDialogueContextMapping& ContextMapping = DialogueWave->ContextMappings[i];

			Sound = DialogueWave->GetWaveFromContext(ContextMapping.Context);
			if (Sound != nullptr)
			{
				break;
			}
		}

		if (Sound)
		{
			GEditor->PlayPreviewSound(Sound);
		}
		else
		{
			StopSound();
		}
	}

	static void ExecutePlaySound(const FToolMenuContext& InContext)
	{
		if (UDialogueWave* DialogueWave = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UDialogueWave>(InContext))
		{
			// Only play the first valid sound
			PlaySound(DialogueWave);
		}
	}

	static void ExecuteStopSound(const FToolMenuContext& InContext)
	{
		StopSound();
	}

	static bool IsCreateSoundCueActionVisible(const FToolMenuContext& InContext, bool bInAcceptMultipleEntries)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			TArray<FAssetData> AssetData = CBContext->SelectedAssets;
			if (AssetData.IsEmpty())
			{
				return false;
			}
			
			// If multiple entries are accepted only return true if the entries are more than 1
			return bInAcceptMultipleEntries ? AssetData.Num() > 1 : AssetData.Num() == 1;
		}

		return false;
	}

	static void ExecuteCreateSoundCue(const FToolMenuContext& InContext, bool bCreateCueForEachDialogueWave)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			TArray<UDialogueWave*> Objects = CBContext->LoadSelectedObjects<UDialogueWave>();

			if (Objects.IsEmpty())
			{
				return;
			}

			const FString DefaultSuffix = TEXT("_Cue");

			if (Objects.Num() == 1 || !bCreateCueForEachDialogueWave)
			{
				UDialogueWave* Object = Objects[0];

				if (Object)
				{
					// Determine an appropriate name
					FString Name;
					FString PackagePath;
					AnimationEditorUtils::CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

					// Create the factory used to generate the asset
					USoundCueFactoryNew* Factory = NewObject<USoundCueFactoryNew>();
					Factory->InitialDialogueWaves = { TWeakObjectPtr<UDialogueWave>(Object) };

					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), USoundCue::StaticClass(), Factory);
				}
			}
			else
			{
				TArray<UObject*> ObjectsToSync;

				for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
				{
					UDialogueWave* Object = (*ObjIt);
					if (Object)
					{
						FString Name;
						FString PackageName;
						AnimationEditorUtils::CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

						// Create the factory used to generate the asset
						USoundCueFactoryNew* Factory = NewObject<USoundCueFactoryNew>();
						Factory->InitialDialogueWaves.Add(Object);

						FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
						UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), USoundCue::StaticClass(), Factory);

						if (NewAsset)
						{
							ObjectsToSync.Add(NewAsset);
						}
					}
				}

				if (ObjectsToSync.Num() > 0)
				{
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
				}
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			
			// Dialogue Wave Action Registration
			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UDialogueWave::StaticClass());

				FToolMenuSection& PlaybackSection = Menu->FindOrAddSection("GetAssetActions");
				PlaybackSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
						{
							if (Context->SelectedAssets.Num() > 0)
							{
								const FAssetData& AssetData = Context->SelectedAssets[0];
								if (AssetData.AssetClassPath == UDialogueWave::StaticClass()->GetClassPathName())
								{
									// Play
									{
										const TAttribute<FText> Label = LOCTEXT("DialogWave_PlaySound", "Play");
										const TAttribute<FText> ToolTip = LOCTEXT("DialogWave_PlaySoundTooltip", "Plays the selected dialog wave.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Play.Small");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecutePlaySound);
										UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecutePlayCommand);
										InSection.AddMenuEntry("DialogWave_PlaySound", Label, ToolTip, Icon, UIAction);
									}

									// Stop
									{
										const TAttribute<FText> Label = LOCTEXT("DialogWave_Stop", "Stop");
										const TAttribute<FText> ToolTip = LOCTEXT("DialogWave_StopTooltip", "Stops the selected dialog wave(s).");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Stop.Small");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteStopSound);
										InSection.AddMenuEntry("DialogWave_StopSound", Label, ToolTip, Icon, UIAction);
									}

									// Create Cue
									{
										constexpr bool bAcceptMultipleEntries = false;
										constexpr bool bCreateCueForEachDialogueWave = true;
										const TAttribute<FText> Label = LOCTEXT("DialogueWave_CreateCue", "Create Cue");
										const TAttribute<FText> ToolTip = LOCTEXT("DialogueWave_CreateCueTooltip", "Creates a sound cue using this dialogue wave.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCue, bCreateCueForEachDialogueWave);
										UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsCreateSoundCueActionVisible, bAcceptMultipleEntries);
										InSection.AddMenuEntry("DialogueWave_CreateCue", Label, ToolTip, Icon, UIAction);
									}

									// Create Single Cue
									{
										constexpr bool bAcceptMultipleEntries = true;
										constexpr bool bCreateCueForEachDialogueWave = false;
										const TAttribute<FText> Label = LOCTEXT("DialogueWave_CreateSingleCue", "Create Single Cue");
										const TAttribute<FText> ToolTip = LOCTEXT("DialogueWave_CreateSingleCueTooltip", "Creates a single sound cue using these dialogue waves.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCue, bCreateCueForEachDialogueWave);
										UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsCreateSoundCueActionVisible, bAcceptMultipleEntries);
										InSection.AddMenuEntry("DialogueWave_CreateSingleCue", Label, ToolTip, Icon, UIAction);
									}

									// Create Multiple Cue
									{
										constexpr bool bAcceptMultipleEntries = true;
										constexpr bool bCreateCueForEachDialogueWave = true;
										const TAttribute<FText> Label = LOCTEXT("DialogueWave_CreateMultipleCue", "Create Multiple Cues");
										const TAttribute<FText> ToolTip = LOCTEXT("DialogueWave_CreateMultipleCueTooltip", "Creates multiple sound cues, one from each dialogue wave.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCue, bCreateCueForEachDialogueWave);
										UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsCreateSoundCueActionVisible, bAcceptMultipleEntries);
										InSection.AddMenuEntry("DialogueWave_CreateMultipleCue", Label, ToolTip, Icon, UIAction);
									}
								}
							}
						}
					}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
