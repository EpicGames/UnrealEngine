// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "Sound/DialogueWave.h"
#include "AssetDefinition_DialogueWave.generated.h"

UCLASS()
class UAssetDefinition_DialogueWave : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_DialogueWave", "Dialogue Wave"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(97, 85, 212); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDialogueWave::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_DialogueWaveSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_DialogueWaveSubMenuSection", "Dialogue"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
