// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "CoreMinimal.h"
#include "Sound/ReverbEffect.h"

#include "AssetDefinition_ReverbEffect.generated.h"

UCLASS()
class UAssetDefinition_ReverbEffect : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ReverbEffect", "Reverb Effect"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UReverbEffect::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetTypeActions", "AssetEffectSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};
