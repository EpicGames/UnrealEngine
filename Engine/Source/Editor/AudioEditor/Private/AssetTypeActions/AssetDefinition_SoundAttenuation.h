// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "CoreMinimal.h"
#include "Sound/SoundAttenuation.h"
#include "AudioEditorSettings.h"

#include "AssetDefinition_SoundAttenuation.generated.h"

UCLASS()
class UAssetDefinition_SoundAttenuation : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundAttenuation", "Attenuation"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(77, 120, 239)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundAttenuation::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto PinnedCategories = { EAssetCategoryPaths::Audio };
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetTypeActions", "AssetSoundAttenuationSubMenu", "Property Settings")) };
		
		if (GetDefault<UAudioEditorSettings>()->bPinSoundAttenuationInAssetMenu)
		{
			return PinnedCategories;
		}
		return Categories;
	}
	// UAssetDefinition End
};
