// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#include "AudioEditorSettings.h"
#include "ContentBrowserMenuContexts.h"
#include "Components/AudioComponent.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundWave.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "AssetDefinitionRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SoundBase)

#define LOCTEXT_NAMESPACE "AudioEditor"

namespace UE::AudioEditor
{
	void StopSound()
	{
		GEditor->ResetPreviewAudioComponent();
	}
	
	void PlaySound(USoundBase* Sound)
	{
		if ( Sound )
		{
			GEditor->PlayPreviewSound(Sound);
		}
		else
		{
			StopSound();
		}
	}
	
	bool IsSoundPlaying(USoundBase* Sound)
	{
		UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
		return PreviewComp && PreviewComp->Sound == Sound && PreviewComp->IsPlaying();
	}
	
	bool IsSoundPlaying(const FAssetData& AssetData)
	{
		const UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
		if (PreviewComp && PreviewComp->Sound && PreviewComp->IsPlaying())
		{
			if (PreviewComp->Sound->GetFName() == AssetData.AssetName)
			{
				if (PreviewComp->Sound->GetOutermost()->GetFName() == AssetData.PackageName)
				{
					return true;
				}
			}
		}
	
		return false;
	}
} // namespace UE::AudioEditor

TArray<FToolMenuSection*> UAssetDefinition_SoundAssetBase::RebuildSoundContextMenuSections() const
{
	const UAudioEditorSettings* EdSettings = GetDefault<UAudioEditorSettings>();
	if (!EdSettings)
	{
		return { };
	}

	const UEnum* EnumClass = StaticEnum<EToolMenuInsertType>();
	check(EnumClass);
	const EToolMenuInsertType InsertType = static_cast<EToolMenuInsertType>(EnumClass->GetValueByName(EdSettings->MenuPosition));

	UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(GetAssetClass());
	check(Menu);

	FToolMenuSection& SoundSection = Menu->FindOrAddSection("Sound");
	SoundSection.Label = LOCTEXT("SoundActions_Label", "Sound");

	const FToolMenuEntry* PositionEntry = SoundSection.FindEntry("PositionActions");
	if (!PositionEntry)
	{
		SoundSection.AddDynamicEntry("PositionActions", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			const TAttribute<FText> Label = TAttribute<FText>::CreateLambda([]()
			{
				if (const UAudioEditorSettings* EdSettings = GetDefault<UAudioEditorSettings>())
				{
					if (EdSettings->MenuPosition == "Last")
					{
						return LOCTEXT("Sound_MenuPositionSendToTop", "Move To Top");
					}
				}
				return LOCTEXT("Sound_MenuPositionSendToBottom", "Move To Bottom");
			});
			const TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>::CreateLambda([]()
			{
				if (const UAudioEditorSettings* EdSettings = GetDefault<UAudioEditorSettings>())
				{
					if (EdSettings->MenuPosition == "Last")
					{
						return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowUp");
					}
				}
				return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowDown");
			});

			const TAttribute<FText> ToolTip = LOCTEXT("Sound_MoveActionsTooltip", "Sets where in this right-click menu to place sound-related actions (includes playback, etc.).");
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext&)
			{
				if (UAudioEditorSettings* EdSettings = GetMutableDefault<UAudioEditorSettings>())
				{
					EdSettings->MenuPosition = EdSettings->MenuPosition == "Last" ? "First" : "Last";
					EdSettings->RebuildSoundContextMenuSections();
				}
			});
			UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda([](const FToolMenuContext&)
			{
				return FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
			});
			InSection.AddMenuEntry("Sound_SortLocation", Label, ToolTip, Icon, UIAction);
		}));
	}

	return TArray<FToolMenuSection*> { &SoundSection };
}

FToolMenuSection* UAssetDefinition_SoundAssetBase::FindSoundContextMenuSection(FName SectionName) const
{
	TArray<FToolMenuSection*> Sections = RebuildSoundContextMenuSections();
	for (FToolMenuSection* Section : Sections)
	{
		if (Section->Name == SectionName)
		{
			return Section;
		}
	}

	return nullptr;
}

EAssetCommandResult UAssetDefinition_SoundBase::ActivateSoundBase(const FAssetActivateArgs& ActivateArgs)
{
	if (ActivateArgs.ActivationMethod == EAssetActivationMethod::Previewed)
	{
		if (USoundBase* TargetSound = ActivateArgs.LoadFirstValid<USoundBase>())
		{
			UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
			if (PreviewComp && PreviewComp->IsPlaying())
			{
				// Already previewing a sound, if it is the target cue then stop it, otherwise play the new one
				if (!TargetSound || PreviewComp->Sound == TargetSound)
				{
					UE::AudioEditor::StopSound();
				}
				else
				{
					UE::AudioEditor::PlaySound(TargetSound);
				}
			}
			else
			{
				// Not already playing, play the target sound cue if it exists
				UE::AudioEditor::PlaySound(TargetSound);
			}

			return EAssetCommandResult::Handled;
		}
	}
	return EAssetCommandResult::Unhandled;
}

TSharedPtr<SWidget> UAssetDefinition_SoundBase::GetSoundBaseThumbnailOverlay(const FAssetData& InAssetData, TFunction<FReply()>&& OnClicked)
{
	auto OnGetDisplayBrushLambda = [InAssetData]() -> const FSlateBrush*
	{
		if (UE::AudioEditor::IsSoundPlaying(InAssetData))
		{
			return FAppStyle::GetBrush("MediaAsset.AssetActions.Stop.Large");
		}

		return FAppStyle::GetBrush("MediaAsset.AssetActions.Play.Large");
	};

	auto OnClickedLambda = MoveTemp(OnClicked);
	if (!OnClickedLambda)
	{
		OnClickedLambda = [InAssetData]() -> FReply
		{
			if (UE::AudioEditor::IsSoundPlaying(InAssetData))
			{
				UE::AudioEditor::StopSound();
			}
			else
			{
				// Load and play sound
				UE::AudioEditor::PlaySound(Cast<USoundBase>(InAssetData.GetAsset()));
			}
			return FReply::Handled();
		};
	}

	auto OnToolTipTextLambda = [InAssetData]() -> FText
	{
		if (UE::AudioEditor::IsSoundPlaying(InAssetData))
		{
			return LOCTEXT("Thumbnail_StopSoundToolTip", "Stop selected sound");
		}

		return LOCTEXT("Thumbnail_PlaySoundToolTip", "Play selected sound");
	};

	TSharedPtr<SBox> Box;
	SAssignNew(Box, SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2));

	auto OnGetVisibilityLambda = [Box, InAssetData]() -> EVisibility
	{
		if (Box.IsValid() && (Box->IsHovered() || UE::AudioEditor::IsSoundPlaying(InAssetData)))
		{
			return EVisibility::Visible;
		}

		return EVisibility::Hidden;
	};

	TSharedPtr<SButton> Widget;
	SAssignNew(Widget, SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ToolTipText_Lambda(OnToolTipTextLambda)
		.Cursor(EMouseCursor::Default) // The outer widget can specify a DragHand cursor, so we need to override that here
		.ForegroundColor(FSlateColor::UseForeground())
		.IsFocusable(false)
		.OnClicked_Lambda(OnClickedLambda)
		.Visibility_Lambda(OnGetVisibilityLambda)
		[
			SNew(SImage)
			.Image_Lambda(OnGetDisplayBrushLambda)
		];

	Box->SetContent(Widget.ToSharedRef());
	Box->SetVisibility(EVisibility::Visible);

	return Box;
}


EAssetCommandResult UAssetDefinition_SoundBase::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	if (ActivateSoundBase(ActivateArgs) == EAssetCommandResult::Handled)
	{
		return EAssetCommandResult::Handled;
	}
	return Super::ActivateAssets(ActivateArgs);
}

void UAssetDefinition_SoundBase::GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const
{
	UAssetDefinition_SoundBase::GetSoundBaseAssetActionButtonExtensions(InAssetData, OutExtensions);
}

TSoftClassPtr<UObject> UAssetDefinition_SoundBase::GetAssetClass() const
{
	return USoundBase::StaticClass();
}

FText UAssetDefinition_SoundBase::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_SoundBase", "Sound Base");
}

FLinearColor UAssetDefinition_SoundBase::GetAssetColor() const
{
	return FLinearColor(FColor(97, 85, 212));
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SoundBase::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Audio };
	return Categories;
}

TSharedPtr<SWidget> UAssetDefinition_SoundBase::GetThumbnailOverlay(const FAssetData& InAssetData) const
{
	auto OnClickedLambda = [InAssetData]() -> FReply
	{
		if (UE::AudioEditor::IsSoundPlaying(InAssetData))
		{
			UE::AudioEditor::StopSound();
		}
		else
		{
			// Load and play sound
			UE::AudioEditor::PlaySound(Cast<USoundBase>(InAssetData.GetAsset()));
		}
		return FReply::Handled();
	};
	return GetSoundBaseThumbnailOverlay(InAssetData, MoveTemp(OnClickedLambda));
}

bool UAssetDefinition_SoundBase::GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const
{
	OutActionOverlayInfo.IsActionPlayingDelegate = FIsActionPlaying::CreateLambda([InAssetData] ()
		{
			return UE::AudioEditor::IsSoundPlaying(InAssetData);
		});

	auto OnToolTipTextLambda = [InAssetData]() -> FText
	{
		if (UE::AudioEditor::IsSoundPlaying(InAssetData))
		{
			return LOCTEXT("Thumbnail_StopSoundToolTip", "Stop selected sound");
		}

		return LOCTEXT("Thumbnail_PlaySoundToolTip", "Play selected sound");
	};

	auto OnClickedLambda = [InAssetData]() -> FReply
	{
		if (UE::AudioEditor::IsSoundPlaying(InAssetData))
		{
			UE::AudioEditor::StopSound();
		}
		else
		{
			// Load and play sound
			UE::AudioEditor::PlaySound(Cast<USoundBase>(InAssetData.GetAsset()));
		}
		return FReply::Handled();
	};

	OutActionOverlayInfo.ActionButtonArgs = SButton::FArguments()
		.ToolTipText_Lambda(OnToolTipTextLambda)
		.OnClicked_Lambda(OnClickedLambda);

	return true;
}

void UAssetDefinition_SoundBase::ExecutePlaySound(const FToolMenuContext& InContext)
{
	if (USoundBase* Sound = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<USoundBase>(InContext))
	{
		// Only play the first valid sound
		UE::AudioEditor::PlaySound(Sound);
	}
}

void UAssetDefinition_SoundBase::ExecuteStopSound(const FToolMenuContext& InContext)
{
	UE::AudioEditor::StopSound();
}

bool UAssetDefinition_SoundBase::CanExecutePlayCommand(const FToolMenuContext& InContext)
{
	if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
	{
		return CBContext->SelectedAssets.Num() == 1;
	}

	return false;
}
	
ECheckBoxState UAssetDefinition_SoundBase::IsActionCheckedMute(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			// If *any* of the selection are muted, show the tick box as ticked.
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();

			for (const FAssetData& SoundCueAsset : CBContext->GetSelectedAssetsOfType(USoundCue::StaticClass()))
			{
				if (Debugger.IsMuteSoundCue(SoundCueAsset.AssetName))
				{
					return ECheckBoxState::Checked;
				}
			}
				
			for (const FAssetData& SoundWaveAsset : CBContext->GetSelectedAssetsOfType(USoundWave::StaticClass()))
			{
				if (Debugger.IsMuteSoundWave(SoundWaveAsset.AssetName))
				{
					return ECheckBoxState::Checked;
				}
			}
		}
	}
#endif
	return ECheckBoxState::Unchecked;
}
	
ECheckBoxState UAssetDefinition_SoundBase::IsActionCheckedSolo(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	// If *any* of the selection are solod, show the tick box as ticked.
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();

			for (const FAssetData& SoundCueAsset : CBContext->GetSelectedAssetsOfType(USoundCue::StaticClass()))
			{
				if (Debugger.IsSoloSoundCue(SoundCueAsset.AssetName))
				{
					return ECheckBoxState::Checked;
				}
			}
				
			for (const FAssetData& SoundWaveAsset : CBContext->GetSelectedAssetsOfType(USoundWave::StaticClass()))
			{
				if (Debugger.IsSoloSoundWave(SoundWaveAsset.AssetName))
				{
					return ECheckBoxState::Checked;
				}
			}
		}
	}
#endif
	return ECheckBoxState::Unchecked;
}

void UAssetDefinition_SoundBase::ExecuteMuteSound(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();

			// In a selection that consists of some already muted, toggle everything in the same direction,
			// to avoid AB problem.
			const bool bAnyMuted = IsActionCheckedMute(InContext) == ECheckBoxState::Checked;

			for (const FAssetData& SoundCueAsset : CBContext->GetSelectedAssetsOfType(USoundCue::StaticClass()))
			{
				Debugger.SetMuteSoundCue(SoundCueAsset.AssetName, !bAnyMuted);
			}
				
			for (const FAssetData& SoundWaveAsset : CBContext->GetSelectedAssetsOfType(USoundWave::StaticClass()))
			{
				Debugger.SetMuteSoundWave(SoundWaveAsset.AssetName, !bAnyMuted);
			}
		}
	}
#endif // ENABLE_AUDIO_DEBUG
}

bool UAssetDefinition_SoundBase::CanExecuteClearMutesAndSolos(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();
			return Debugger.IsMuteOrSoloActive();
		}
	}
#endif // ENABLE_AUDIO_DEBUG

	return false;
}

void UAssetDefinition_SoundBase::ExecuteClearMutesAndSolos(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();
			Debugger.ClearMutesAndSolos();
		}
	}
#endif // ENABLE_AUDIO_DEBUG
}

void UAssetDefinition_SoundBase::ExecuteSoloSound(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();

			// In a selection that consists of some already soloed, toggle everything in the same direction,
			// to avoid AB problem.

			const bool bAnySoloed = IsActionCheckedSolo(InContext) == ECheckBoxState::Checked;

			for (const FAssetData& SoundCueAsset : CBContext->GetSelectedAssetsOfType(USoundCue::StaticClass()))
			{
				Debugger.SetSoloSoundCue(SoundCueAsset.AssetName, !bAnySoloed);
			}
				
			for (const FAssetData& SoundWaveAsset : CBContext->GetSelectedAssetsOfType(USoundWave::StaticClass()))
			{
				Debugger.SetSoloSoundWave(SoundWaveAsset.AssetName, !bAnySoloed);
			}
		}
	}
#endif // ENABLE_AUDIO_DEBUG
}

bool UAssetDefinition_SoundBase::CanExecuteMuteCommand(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			// Allow muting if we're not Soloing.
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();

			for (const FAssetData& SoundCueAsset : CBContext->GetSelectedAssetsOfType(USoundCue::StaticClass()))
			{
				if (Debugger.IsSoloSoundCue(SoundCueAsset.AssetName))
				{
					return false;
				}
			}
				
			for (const FAssetData& SoundWaveAsset : CBContext->GetSelectedAssetsOfType(USoundWave::StaticClass()))
			{
				if (Debugger.IsSoloSoundWave(SoundWaveAsset.AssetName))
				{
					return false;
				}
			}

			// Ok.
			return true;
		}
	}
#endif // ENABLE_AUDIO_DEBUG
	return false;
}

bool UAssetDefinition_SoundBase::CanExecuteSoloCommand(const FToolMenuContext& InContext)
{
#if ENABLE_AUDIO_DEBUG
	if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			// Allow Soloing if not Muting.
			Audio::FAudioDebugger& Debugger = ADM->GetDebugger();
			
			for (const FAssetData& SoundCueAsset : CBContext->GetSelectedAssetsOfType(USoundCue::StaticClass()))
			{
				if (Debugger.IsMuteSoundCue(SoundCueAsset.AssetName))
				{
					return false;
				}
			}

			for (const FAssetData& SoundWaveAsset : CBContext->GetSelectedAssetsOfType(USoundWave::StaticClass()))
			{
				if (Debugger.IsMuteSoundWave(SoundWaveAsset.AssetName))
				{
					return false;
				}
			}

			return true;
		}
	}
#endif
	return false;
}

bool UAssetDefinition_SoundBase::IsPlayingContextAsset(const FToolMenuContext& InContext, bool bMustMatchContext)
{
	const UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent();
	if (!PreviewComponent || !PreviewComponent->IsPlaying())
	{
		return false;
	}

	if (!bMustMatchContext)
	{
		return true;
	}

	if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
	{
		if (CBContext->SelectedAssets.Num() == 1)
		{
			if (const USoundBase* PreviewSound = PreviewComponent->Sound)
			{
				const FAssetData AssetData = CBContext->SelectedAssets.Last();
				if (AssetData.IsAssetLoaded())
				{
					if (UObject* ContextObject = AssetData.GetAsset())
					{
						return PreviewSound->GetUniqueID() == ContextObject->GetUniqueID();
					}
				}
			}
		}
	}

	return false;
}

const USoundBase* UAssetDefinition_SoundBase::GetPlayingSound()
{
	const UAudioComponent* PreviewComponent = GEditor->GetPreviewAudioComponent();
	if (!PreviewComponent || !PreviewComponent->IsPlaying())
	{
		return nullptr;
	}

	return PreviewComponent->Sound.Get();
}

void UAssetDefinition_SoundBase::GetSoundBaseAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions)
{
	FAssetButtonActionExtension AssetButtonActionExtension
	{
		.PickTooltipAttribute = TAttribute<FText>::CreateLambda([InAssetData]() -> const FText
		{
			if (UE::AudioEditor::IsSoundPlaying(InAssetData))
			{
				return LOCTEXT("SoundAudition_PlaySoundToolTip", "Stop selected sound");
			}

			return LOCTEXT("SoundAudition_StopSoundToolTip", "Play selected sound");
		}),
		.PickBrushAttribute = TAttribute<const FSlateBrush*>::CreateLambda([InAssetData]() -> const FSlateBrush* {

			if (UE::AudioEditor::IsSoundPlaying(InAssetData))
			{
				return FAppStyle::GetBrush("MediaAsset.AssetActions.Stop.Small");
			}

			return FAppStyle::GetBrush("MediaAsset.AssetActions.Play.Small");
		}),
		.OnClicked = FOnClicked::CreateLambda([InAssetData]() -> FReply
			{
				if (UE::AudioEditor::IsSoundPlaying(InAssetData))
				{
					UE::AudioEditor::StopSound();
				}
				else
				{
					// Load and play sound
					UE::AudioEditor::PlaySound(Cast<USoundBase>(InAssetData.GetAsset()));
				}

			return FReply::Handled();
		})
	};

	OutExtensions.Add(AssetButtonActionExtension);
}

TArray<FToolMenuSection*> UAssetDefinition_SoundBase::RebuildSoundContextMenuSections() const
{
	const UAudioEditorSettings* EdSettings = GetDefault<UAudioEditorSettings>();
	if (!EdSettings)
	{
		return { };
	}

	TArray<FToolMenuSection*> AssetSections = Super::RebuildSoundContextMenuSections();

	const UEnum* EnumClass = StaticEnum<EToolMenuInsertType>();
	check(EnumClass);
	const EToolMenuInsertType InsertType = static_cast<EToolMenuInsertType>(EnumClass->GetValueByName(EdSettings->MenuPosition));

	UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(GetAssetClass());
	check(Menu);

	FToolMenuSection& PlaySection = Menu->FindOrAddSection("Playback");
	PlaySection.Label = LOCTEXT("PlaybackActions_Label", "Playback");

	if (InsertType == EToolMenuInsertType::Last)
	{
		AssetSections.Last()->InsertPosition = FToolMenuInsert("Playback", EToolMenuInsertType::Before);
		PlaySection.InsertPosition = FToolMenuInsert(FName(), EToolMenuInsertType::Last);
	}
	else
	{
		PlaySection.InsertPosition = FToolMenuInsert(FName(), EToolMenuInsertType::First);
		AssetSections.Last()->InsertPosition = FToolMenuInsert("Playback", EToolMenuInsertType::After);
	}

	AssetSections.Add(&PlaySection);
	return AssetSections;
}

namespace MenuExtension_SoundBase
{
	static void RegisterAssetActions(UClass* InClass)
	{
		const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(InClass);

		FToolMenuSection* PlaySection = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Playback");
		check(PlaySection);

		PlaySection->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
			{
				if (Context->SelectedAssets.Num() > 0)
				{
					const FAssetData& AssetData = Context->SelectedAssets[0];
					if (AssetData.AssetClassPath == USoundWave::StaticClass()->GetClassPathName() || AssetData.AssetClassPath == USoundCue::StaticClass()->GetClassPathName())
					{
						auto IsPlayingThis = [](const FToolMenuContext& InContext) { return UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, true /* bMustMatchContext */); };
						auto IsNotPlayingThis = [](const FToolMenuContext& InContext) { return !UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, true /* bMustMatchContext */); };
						auto IsNotPlayingAny = [](const FToolMenuContext& InContext) { return !UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, false /* bMustMatchContext */); };
						auto IsPlayingAny = [](const FToolMenuContext& InContext) { return UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, false /* bMustMatchContext */); };
						auto IsPlayingOther = [](const FToolMenuContext& InContext) { return UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, false /* bMustMatchContext */) && !UAssetDefinition_SoundBase::IsPlayingContextAsset(InContext, true /* bMustMatchContext */); };

						{
							const TAttribute<FText> Label = LOCTEXT("Sound_PlaySound", "Play");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_PlaySoundTooltip", "Plays the selected sound.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Play.Small");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecutePlaySound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::CanExecutePlayCommand);
							UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda(IsNotPlayingThis);
							InSection.AddMenuEntry("Sound_PlaySound", Label, ToolTip, Icon, UIAction);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_RestartSound", "Restart");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_RestartSoundTooltip", "Restarts the selected sound.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cascade.RestartInLevel.Small");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecutePlaySound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::CanExecutePlayCommand);
							UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda(IsPlayingThis);
							InSection.AddMenuEntry("Sound_RestartSound", Label, ToolTip, Icon, UIAction);
						}
						{ // Stop
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Stop.Small");
							{ // Selected
								const TAttribute<FText> StopSelectedToolTip = LOCTEXT("Sound_StopSoundTooltip", "Stops the selected sound.");
								{
									const TAttribute<FText> Label = LOCTEXT("Sound_StopSoundDisabled", "Stop");

									FToolUIAction UIAction;
									UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(IsPlayingThis);
									UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecuteStopSound);
									UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda([IsNotPlayingAny, IsPlayingThis](const FToolMenuContext& Context) { return IsNotPlayingAny(Context) || IsPlayingThis(Context); });
									InSection.AddMenuEntry("Sound_StopSound", Label, StopSelectedToolTip, Icon, UIAction);
								}
							}
							{ // Other
								const TAttribute<FText> Label = TAttribute<FText>::CreateLambda([]()
								{
									if (const USoundBase* OtherSound = UAssetDefinition_SoundBase::GetPlayingSound())
									{
										return FText::Format(LOCTEXT("Sound_StopSoundOtherFormat", "Stop ({0})"), FText::FromName(OtherSound->GetFName()));
									}
									return LOCTEXT("Sound_StopSoundOther", "Stop (Other)");
								});
								const TAttribute<FText> ToolTip = LOCTEXT("Sound_StopOtherSoundTooltip", "Stops the currently previewing (other) sound.");
								FToolUIAction UIAction;
								UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecuteStopSound);
								UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda(IsPlayingOther);
								InSection.AddMenuEntry("Sound_StopOtherSound", Label, ToolTip, Icon, UIAction);
							}
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_MuteSound", "Mute");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_MuteSoundTooltip", "Mutes the selected sounds.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Mute.Small");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecuteMuteSound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::CanExecuteMuteCommand);
							UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&UAssetDefinition_SoundBase::IsActionCheckedMute);
							InSection.AddMenuEntry("Sound_SoundMute", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::ToggleButton);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_SoloSound", "Solo");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_SoloSoundTooltip", "Solos the selected sounds.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Solo.Small");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecuteSoloSound);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::CanExecuteSoloCommand);
							UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&UAssetDefinition_SoundBase::IsActionCheckedSolo);
							InSection.AddMenuEntry("Sound_StopSolo", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::ToggleButton);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Sound_ClearMutedSoloed", "Clear Muted/Soloed");
							const TAttribute<FText> ToolTip = LOCTEXT("Sound_ClearMutedSoloedTooltip", "Clear all flags to mute/solo specific assets.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.DiffersFromDefault");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::ExecuteClearMutesAndSolos);
							UIAction.IsActionVisibleDelegate = FToolMenuCanExecuteAction::CreateStatic(&UAssetDefinition_SoundBase::CanExecuteClearMutesAndSolos);
							InSection.AddMenuEntry("Sound_ClearMuteSoloSettings", Label, ToolTip, Icon, UIAction);
						}
					}
				}
			}
		}));
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			// Note: we can't do this with USoundBase because UMetaSoundSource and others may need to do special actions on these actions.
			// Since these actions are registered via delayed static callbacks, there's not a clean why to do this with inheritance.
			RegisterAssetActions(USoundCue::StaticClass());
			RegisterAssetActions(USoundWave::StaticClass());
		}));
	});
}

#undef LOCTEXT_NAMESPACE // AudioEditor
