// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SoundClass.h"

#include "AudioDeviceManager.h"
#include "AudioEditorModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "Audio/AudioDebug.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_SoundClass"

EAssetCommandResult UAssetDefinition_SoundClass::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (USoundClass* SoundClass : OpenArgs.LoadObjects<USoundClass>())
	{
		IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );
		AudioEditorModule->CreateSoundClassEditor(Mode, OpenArgs.ToolkitHost, SoundClass);
	}
	return EAssetCommandResult::Handled;
}

namespace MenuExtension::SoundClass
{
	static void ExecuteMute(const FToolMenuContext& InContext)
	{
#if ENABLE_AUDIO_DEBUG
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
		{
			for (USoundClass* SoundClass : CBContext->LoadSelectedObjects<USoundClass>())
			{
				ADM->GetDebugger().ToggleMuteSoundClass(SoundClass->GetFName());
			}
		}
#endif
	}
	
	static void ExecuteSolo(const FToolMenuContext& InContext)
	{
#if ENABLE_AUDIO_DEBUG
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
		{
			for (USoundClass* SoundClass : CBContext->LoadSelectedObjects<USoundClass>())
			{
				ADM->GetDebugger().ToggleSoloSoundClass(SoundClass->GetFName());
			}
		}
#endif
	}
	
	static ECheckBoxState IsActionCheckedMute(const FToolMenuContext& InContext)
	{
#if ENABLE_AUDIO_DEBUG
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return ECheckBoxState::Undetermined;
		}

		if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
		{
			for (USoundClass* SoundClass : CBContext->LoadSelectedObjects<USoundClass>())
			{
				if (ADM->GetDebugger().IsMuteSoundClass(SoundClass->GetFName()))
				{
					return ECheckBoxState::Checked;
				}
			}
		}
#endif
		return ECheckBoxState::Unchecked;
	}
	
	static ECheckBoxState IsActionCheckedSolo(const FToolMenuContext& InContext)
	{
#if ENABLE_AUDIO_DEBUG
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return ECheckBoxState::Undetermined;
		}

		if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
		{
			for (USoundClass* SoundClass : CBContext->LoadSelectedObjects<USoundClass>())
			{
				if (ADM->GetDebugger().IsSoloSoundClass(SoundClass->GetFName()))
				{
					return ECheckBoxState::Checked;
				}
			}
		}
#endif
		return ECheckBoxState::Unchecked;
	}
	
	static bool CanExecuteMuteCommand(const FToolMenuContext& InContext)
	{
#if ENABLE_AUDIO_DEBUG
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return false;
		}

		if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
		{
			// Allow muting if we're not Soloing.
			for (USoundClass* SoundClass : CBContext->LoadSelectedObjects<USoundClass>())
			{
				if (ADM->GetDebugger().IsSoloSoundClass(SoundClass->GetFName()))
				{
					return false;
				}
			}
			// Ok.
			return true;
		}
#endif
		return false;
	}
	
	static bool CanExecuteSoloCommand(const FToolMenuContext& InContext)
	{
#if ENABLE_AUDIO_DEBUG
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return false;
		}

		if (FAudioDeviceManager* ADM = GEditor->GetAudioDeviceManager())
		{
			// Allow Soloing if we're not Muting.
			for (USoundClass* SoundClass : CBContext->LoadSelectedObjects<USoundClass>())
			{
				if (ADM->GetDebugger().IsMuteSoundClass(SoundClass->GetFName()))
				{
					return false;
				}
			}
			// Ok.
			return true;
		}
#endif
		return false;
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			
			// Sound Class Action Registration
			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USoundClass::StaticClass());

				FToolMenuSection& PlaybackSection = Menu->FindOrAddSection("GetAssetActions");
				PlaybackSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
						{
							if (Context->SelectedAssets.Num() > 0)
							{
								const FAssetData& AssetData = Context->SelectedAssets[0];
								if (AssetData.AssetClassPath == USoundClass::StaticClass()->GetClassPathName())
								{
									// Mute
									{
										const TAttribute<FText> Label = LOCTEXT("Sound_MuteSoundClass", "Mute");
										const TAttribute<FText> ToolTip = LOCTEXT("Sound_MuteSoundClassTooltip", "Mutes anything using this SoundClass");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Mute.Small");
										constexpr EUserInterfaceActionType ActionType = EUserInterfaceActionType::ToggleButton;

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteMute);
										UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteMuteCommand);
										UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&IsActionCheckedMute);
										InSection.AddMenuEntry("SoundClass_Mute", Label, ToolTip, Icon, UIAction, ActionType);
									}

									// Solo
									{
										const TAttribute<FText> Label = LOCTEXT("Sound_SoloSoundClass", "Solo");
										const TAttribute<FText> ToolTip = LOCTEXT("Sound_SoloSoundClassTooltip", "Solos the selected sounds.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Solo.Small");
										constexpr EUserInterfaceActionType ActionType = EUserInterfaceActionType::ToggleButton;

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteSolo);
										UIAction.CanExecuteAction = FToolMenuIsActionButtonVisible::CreateStatic(&CanExecuteSoloCommand);
										UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&IsActionCheckedSolo);
										InSection.AddMenuEntry("SoundClass_Solo", Label, ToolTip, Icon, UIAction, ActionType);
									}
								}
							}
						}
					}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
