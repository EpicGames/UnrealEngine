// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "Sound/SoundClass.h"
#include "AssetDefinition_SoundClass.generated.h"

UCLASS()
class UAssetDefinition_SoundClass : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundClass", "Sound Class"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(255, 175, 0); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundClass::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundClassSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundClassSubMenuSection", "Properties"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
