// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "CoreMinimal.h"
#include "Sound/SoundConcurrency.h"
#include "AudioEditorSettings.h"

#include "AssetDefinition_SoundConcurrency.generated.h"

UCLASS()
class UAssetDefinition_SoundConcurrency : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundConcurrency", "Concurrency"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(77, 100, 139)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundConcurrency::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto PinnedCategories = { EAssetCategoryPaths::Audio };
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetTypeActions", "AssetSoundConcurrencySubMenu", "Property Settings")) };

		if (GetDefault<UAudioEditorSettings>()->bPinSoundConcurrencyInAssetMenu)
		{
			return PinnedCategories;
		}
		return Categories;
	}
	// UAssetDefinition End
};
