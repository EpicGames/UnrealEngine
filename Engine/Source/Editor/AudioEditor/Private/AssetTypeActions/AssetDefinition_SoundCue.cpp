// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetDefinition_SoundCue.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "AudioEditorModule.h"
#include "AudioEditorSettings.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "Factories/SoundAttenuationFactory.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundCue.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuSection.h"
#include "Toolkits/IToolkit.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"

#include <utility>

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SoundCue)

class IToolkitHost;
class UClass;

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SoundCue::GetAssetCategories() const
{
	static const auto Pinned_Categories = { EAssetCategoryPaths::Audio };
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, LOCTEXT("SoundCue_SubMenu", "Sources")) };
	
	if (GetDefault<UAudioEditorSettings>()->bPinSoundCueInAssetMenu)
	{
		return Pinned_Categories;
	}
	
	return Categories;
}

EAssetCommandResult UAssetDefinition_SoundCue::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IAudioEditorModule& AudioEditorModule = FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );
	
	for (USoundCue* SoundCue : OpenArgs.LoadObjects<USoundCue>())
	{
		AudioEditorModule.CreateSoundCueEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, SoundCue);
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_SoundCue
{
	bool CanExecuteConsolidateAttenuation(const FToolMenuContext& MenuContext)
	{
		return UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(MenuContext) > 1;
	}

	void ExecuteConsolidateAttenuation(const FToolMenuContext& MenuContext)
	{
		TMap<FSoundAttenuationSettings*, TArray<USoundCue*>> UnmatchedAttenuations;

		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			for (USoundCue* SoundCue : Context->LoadSelectedObjects<USoundCue>())
			{
				bool bFound = false;
				if ( SoundCue && SoundCue->bOverrideAttenuation )
				{
					for (auto UnmatchedIt = UnmatchedAttenuations.CreateIterator(); UnmatchedIt; ++UnmatchedIt)
					{
						// Found attenuation settings to consolidate together
						if (SoundCue->AttenuationOverrides == *UnmatchedIt.Key())
						{
							UnmatchedIt.Value().Add(SoundCue);
							bFound = true;
							break;
						}
					}
					if (!bFound)
					{
						UnmatchedAttenuations.FindOrAdd(&SoundCue->AttenuationOverrides).Add(SoundCue);
					}
				}
			}
		}

		if (UnmatchedAttenuations.Num() > 0)
		{
			FString DefaultSuffix;
			TArray<UObject*> ObjectsToSync;

			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			USoundAttenuationFactory* Factory = NewObject<USoundAttenuationFactory>();

			for (auto UnmatchedIt = UnmatchedAttenuations.CreateConstIterator(); UnmatchedIt; ++UnmatchedIt)
			{
				if (UnmatchedIt.Value().Num() > 1)
				{
					FString Name;
					FString PackageName;
					AssetToolsModule.Get().CreateUniqueAssetName("/Game/Sounds/SoundAttenuations/SharedAttenuation", DefaultSuffix, PackageName, Name);

					USoundAttenuation* SoundAttenuation = Cast<USoundAttenuation>(AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), USoundAttenuation::StaticClass(), Factory));
					if (SoundAttenuation)
					{
						SoundAttenuation->Attenuation = *UnmatchedIt.Key();

						for (int32 SoundCueIndex = 0; SoundCueIndex < UnmatchedIt.Value().Num(); ++SoundCueIndex)
						{
							USoundCue* SoundCue = UnmatchedIt.Value()[SoundCueIndex];
							SoundCue->bOverrideAttenuation = false;
							SoundCue->AttenuationSettings = SoundAttenuation;
							SoundCue->MarkPackageDirty();
						}
					}
				}
			}

			if ( ObjectsToSync.Num() > 0 )
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			{
				const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundCue::StaticClass());
				FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
				Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					const TAttribute<FText> Label = LOCTEXT("SoundCue_ConsolidateAttenuation", "Consolidate Attenuation");
					const TAttribute<FText> ToolTip = LOCTEXT("SoundCue_ConsolidateAttenuationTooltip", "Creates shared attenuation packages for sound cues with identical override attenuation settings.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.SoundConsolidation.Small");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteConsolidateAttenuation);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteConsolidateAttenuation);
					InSection.AddMenuEntry("SoundCue_ConsolidateAttenuation", Label, ToolTip, Icon, UIAction);
				}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
