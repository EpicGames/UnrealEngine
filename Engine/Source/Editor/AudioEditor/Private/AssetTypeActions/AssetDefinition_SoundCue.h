// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundCue.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"

#include "AssetDefinition_SoundCue.generated.h"

UCLASS()
class UAssetDefinition_SoundCue : public UAssetDefinition_SoundBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundCue", "Sound Cue"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 175, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundCue::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
