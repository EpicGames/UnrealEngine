// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SoundEffectPreset.h"

#include "Audio/AudioWidgetSubsystem.h"
#include "Editor.h"
#include "Editors/SoundEffectPresetEditor.h"
#include "Templates/SubclassOf.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "UObject/Class.h"
#include "UObject/UObjectBaseUtility.h"

#define LOCTEXT_NAMESPACE "UAssetDefinition_SoundEffectPresetDynamic"

FText UAssetDefinition_SoundEffectPresetDynamic::GetAssetDisplayName() const
{
	if (!EffectPresetCDO.IsValid())
	{
		return LOCTEXT("AssetDefinition_SoundEffectPresetDynamic", "Sound Effect Preset Base (Invalid)");
	}

	const FText AssetActionName = EffectPresetCDO->GetAssetActionName();
	if (AssetActionName.IsEmpty())
	{
		FString ClassName;
		EffectPresetCDO->GetClass()->GetName(ClassName);
		ensureMsgf(false, TEXT("U%sGetAssetActionName not implemented. Please check that EFFECT_PRESET_METHODS(EffectClassName) is at the top of the declaration of %s."), *ClassName, *ClassName);
		const FString DefaultName = ClassName + FString(TEXT(" (Error: EFFECT_PRESET_METHODS() Not Used in Class Declaration)"));
		return FText::FromString(DefaultName);
	}

	return EffectPresetCDO->GetAssetActionName();
}

FLinearColor UAssetDefinition_SoundEffectPresetDynamic::GetAssetColor() const
{
	if (!EffectPresetCDO.IsValid())
	{
		return FColor::White;
	}

	return EffectPresetCDO->GetPresetColor();
}

TSoftClassPtr<UObject> UAssetDefinition_SoundEffectPresetDynamic::GetAssetClass() const
{
	if (!EffectPresetCDO.IsValid())
	{
		return USoundEffectPreset::StaticClass();
	}

	const UClass* SupportedClass = EffectPresetCDO->GetSupportedClass();
	if (SupportedClass == nullptr)
	{
		FString ClassName;
		EffectPresetCDO->GetClass()->GetName(ClassName);
		ensureMsgf(false, TEXT("U%s::GetSupportedClass not implemented. Please check that EFFECT_PRESET_METHODS(EffectClassName) is at the top of the declaration of %s."), *ClassName, *ClassName);
		return EffectPresetCDO->GetClass();
	}

	return SupportedClass;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SoundEffectPresetDynamic::GetAssetCategories() const
{
	if (!EffectPresetCDO.IsValid() || !EffectPresetCDO->CanFilter())
	{
		static const TArray<FAssetCategoryPath> EmptyCategory = TArray<FAssetCategoryPath>();
		return EmptyCategory;
	}

	static const TArray<FAssetCategoryPath> Categories = 
	{
		FAssetCategoryPath(EAssetCategoryPaths::Audio,
			NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectPresetSubMenu", "Advanced"),
			FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectPresetSubMenuSection", "DSP Effects and Synthesis"), ECategoryMenuType::Section))
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_SoundEffectPresetDynamic::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	if (UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr)
	{
		if (UAudioWidgetSubsystem* WidgetSubsystem = GEngine ? GEngine->GetEngineSubsystem<UAudioWidgetSubsystem>() : nullptr)
		{
			for (int32 Index = Objects.Num() - 1; Index >= 0; --Index)
			{
				USoundEffectPreset* Preset = Cast<USoundEffectPreset>(Objects[Index]);
				if (!Preset)
				{
					continue;
				}

				auto FilterFunction = [InPresetClass = Preset->GetClass()](UUserWidget* UserWidget)
					{
						TSubclassOf<USoundEffectPreset> PresetClass = ISoundEffectPresetWidgetInterface::Execute_GetClass(UserWidget);
						while (PresetClass)
						{
							if (PresetClass == InPresetClass)
							{
								return true;
							}

							PresetClass = PresetClass->GetSuperClass();
						}

						return false;
					};

				TArray<UUserWidget*> UserWidgets = WidgetSubsystem->CreateUserWidgets(*World, USoundEffectPresetWidgetInterface::StaticClass(), FilterFunction);
				if (!UserWidgets.IsEmpty())
				{
					TSharedRef<FSoundEffectPresetEditor> PresetEditor = MakeShared<FSoundEffectPresetEditor>();
					PresetEditor->Init(Mode, OpenArgs.ToolkitHost, Preset, UserWidgets);
					Objects.RemoveAt(Index);
				}
			}
		}
	}

	// Presets with no matching widget blueprint fall back to the default properties-only editor.
	for (UObject* Object : Objects)
	{
		if (USoundEffectPreset* Preset = Cast<USoundEffectPreset>(Object))
		{
			FSimpleAssetEditor::CreateEditor(Mode, OpenArgs.ToolkitHost, Preset);
		}
	}

	return EAssetCommandResult::Handled;
}

void UAssetDefinition_SoundEffectPresetDynamic::Initialize(TSubclassOf<USoundEffectPreset> InClass)
{
	EffectPresetCDO = TStrongObjectPtr(InClass->GetDefaultObject<USoundEffectPreset>());
}

#undef LOCTEXT_NAMESPACE
