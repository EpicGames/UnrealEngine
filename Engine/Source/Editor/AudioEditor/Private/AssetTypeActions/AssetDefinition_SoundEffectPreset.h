// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/SoundEffectSubmix.h"
#include "AssetDefinition_SoundEffectPreset.generated.h"

UCLASS()
class UAssetDefinition_SoundEffectSubmixPreset : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSubmixPreset", "Submix Effect Preset"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(99, 63, 56); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundEffectSubmixPreset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSubmixPresetSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSubmixPresetSubMenuSection", "DSP Effects and Synthesis"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundEffectSourcePreset : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSourcePreset", "Source Effect Preset"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(72, 185, 187); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundEffectSourcePreset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSourcePresetSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSourcePresetSubMenuSection", "DSP Effects and Synthesis"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundEffectSourcePresetChain : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSourcePresetChain", "Source Effect Preset Chain"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(51, 107, 142); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundEffectSourcePresetChain::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSourcePresetChainSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundEffectSourcePresetChainSubMenuSection", "DSP Effects and Synthesis"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundEffectPresetDynamic : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual bool CanRegisterStatically() const override { return false; }
	// UAssetDefinition End

	void Initialize(TSubclassOf<USoundEffectPreset> InClass);

private:
	TStrongObjectPtr<USoundEffectPreset> EffectPresetCDO;
};
