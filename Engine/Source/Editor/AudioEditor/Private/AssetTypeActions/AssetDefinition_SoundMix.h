// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "CoreMinimal.h"
#include "Sound/SoundMix.h"

#include "AssetDefinition_SoundMix.generated.h"

UCLASS()
class UAssetDefinition_SoundMix : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundMix", "Sound Class Mix"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundMix::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "SoundMix_SubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};
