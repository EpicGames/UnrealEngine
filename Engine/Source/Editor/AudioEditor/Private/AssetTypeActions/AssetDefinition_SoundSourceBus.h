// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Sound/SoundSourceBus.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"

#include "AssetDefinition_SoundSourceBus.generated.h"

UCLASS()
class UAssetDefinition_SoundSourceBus : public UAssetDefinition_SoundBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundSourceBus", "Source Bus"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(212, 97, 85)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundSourceBus::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio,
					NSLOCTEXT("AssetDefinition", "AssetDefinitionSoundSourceSubMenu", "Advanced"),
					FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundSourceSubMenuSection", "Routing"), ECategoryMenuType::Section))
			};
		return Categories;
	}
	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& InAssetData) const override { return nullptr; }
	virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const override { return false; }
	// UAssetDefinition End
};
