// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SoundSubmix.h"

#include "AudioEditorModule.h"
#include "Editor.h"
#include "SoundSubmixEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"

EAssetCommandResult UAssetDefinition_SoundSubmix::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	TArray<UObject*> Objects = ActivateArgs.LoadObjects<UObject>();
	TSet<USoundSubmixBase*> SubmixesToSelect;
	IAssetEditorInstance* Editor = nullptr;
	for (UObject* Obj : Objects)
	{
		if (USoundSubmixBase* SubmixToSelect = Cast<USoundSubmixBase>(Obj))
		{
			if (!Editor)
			{
				Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Obj, true);
			}
			SubmixesToSelect.Add(SubmixToSelect);
		}
	}

	if (Editor)
	{
		static_cast<FSoundSubmixEditor*>(Editor)->SelectSubmixes(SubmixesToSelect);
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

EAssetCommandResult UAssetDefinition_SoundSubmix::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UObject* Obj : Objects)
	{
		if (USoundSubmixBase* SoundSubmix = Cast<USoundSubmixBase>(Obj))
		{
			IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
			AudioEditorModule->CreateSoundSubmixEditor(Mode, OpenArgs.ToolkitHost, SoundSubmix);
		}
	}

	return EAssetCommandResult::Handled;
}
