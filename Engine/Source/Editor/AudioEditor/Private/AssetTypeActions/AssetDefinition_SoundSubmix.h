// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "SoundSubmixDefaultColorPalette.h"
#include "AssetDefinition_SoundSubmix.generated.h"

UCLASS()
class UAssetDefinition_SoundSubmix : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundSubmix", "Sound Submix"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::DefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundSubmix::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio,
					NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundSubmixSubMenu", "Advanced"),
					FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundSubmixSubMenuSection", "Routing"), ECategoryMenuType::Section))
			};
		return Categories;
	}
	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundfieldSubmix : public UAssetDefinition_SoundSubmix
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldSubmix", "Soundfield Submix"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::SoundfieldDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundfieldSubmix::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldSubmixSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_EndpointSubmix : public UAssetDefinition_SoundSubmix
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_EndpointSubmix", "Endpoint Submix"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::EndpointDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UEndpointSubmix::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_EndpointSubmixSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundfieldEndpointSubmix : public UAssetDefinition_SoundSubmix
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEndpointSubmix", "Soundfield Endpoint Submix"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::SoundfieldEndpointDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundfieldEndpointSubmix::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEndpointSubmixSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundfieldEncodingSettings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEncodingSettings", "Soundfield Encoding Settings"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::SoundfieldDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundfieldEncodingSettingsBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEncodingSettingsSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundfieldEffectSettings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEffectSettings", "Soundfield Effect Settings"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::SoundfieldDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundfieldEffectSettingsBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEffectSettingsSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundfieldEffect : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEffect", "Soundfield Effect"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::SoundfieldDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundfieldEffectBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEffectSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_AudioEndpointSettings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioEndpointSettings", "Audio Endpoint Settings"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::EndpointDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioEndpointSettingsBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioEndpointSettingsSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SoundfieldEndpointSettings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEndpointSettings", "Soundfield Endpoint Settings"); }
	virtual FLinearColor GetAssetColor() const override { return Audio::EndpointDefaultSubmixColor; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundfieldEndpointSettingsBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundfieldEndpointSettingsSubMenu", "Legacy")) };
		return Categories;
	}
	// UAssetDefinition End
};
