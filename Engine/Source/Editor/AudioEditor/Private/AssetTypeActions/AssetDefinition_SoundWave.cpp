// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetDefinition_SoundWave.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "AudioEditorModule.h"
#include "AudioEditorSettings.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "Factories/DialogueWaveFactory.h"
#include "Factories/SoundAttenuationFactory.h"
#include "Factories/SoundCueFactoryNew.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "Sound/DialogueVoice.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundCue.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "ContentBrowserMenuContexts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SoundWave)

#define LOCTEXT_NAMESPACE "AssetDefinition_SoundWave"

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SoundWave::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Audio / LOCTEXT("AssetSoundWaveSubMenu", "Source") };
	
	return Categories;
}

FText UAssetDefinition_SoundWave::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_SoundWave", "Sound Wave");
}

FLinearColor UAssetDefinition_SoundWave::GetAssetColor() const
{
	return FLinearColor(FColor(97, 85, 212));
}

TSoftClassPtr<UObject> UAssetDefinition_SoundWave::GetAssetClass() const
{
	return USoundWave::StaticClass();
}

bool UAssetDefinition_SoundWave::CanImport() const
{
	return true;
}

EAssetCommandResult UAssetDefinition_SoundWave::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IAudioEditorModule& AudioEditorModule = FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );

	// Use a custom editor for soundwaves if one is registered except in restricted mode
	// which currently does not support custom soundwave editors
	if (AudioEditorModule.SoundWaveEditorOpen.IsBound() && !AudioEditorModule.IsRestrictedMode())
	{
		AudioEditorModule.SoundWaveEditorOpen.Execute(OpenArgs.LoadObjects<USoundWave>());
	}
	else // Otherwise open a default editor
	{
		for (USoundWave* SoundWave : OpenArgs.LoadObjects<USoundWave>())
		{
			FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, SoundWave);
		}
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------
namespace MenuExtension_SoundWave
{
	/** Creates a unique package and asset name taking the form InBasePackageName+InSuffix */
	void CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(InBasePackageName, InSuffix, OutPackageName, OutAssetName);
	}

	void ExecuteCreateDialogueWave(const struct FAssetData& InAssetData, TArray<TSoftObjectPtr<USoundWave>> InSoftObjects)
	{
		const FString DefaultSuffix = TEXT("_Dialogue");

		UDialogueVoice* DialogueVoice = Cast<UDialogueVoice>(InAssetData.GetAsset());

		TArray<TWeakObjectPtr<USoundWave>> Objects;

		//Load the selected objects to memory
		for (TSoftObjectPtr<USoundWave>& SoundWave : InSoftObjects)
		{		
			if (USoundWave* SoundWaveObject = SoundWave.LoadSynchronous())
			{
				Objects.Add(SoundWaveObject);
			}
		}

		if (Objects.Num() == 1)
		{
			USoundWave* Object = Objects.Last().Get();

			if (Object)
			{
				// Determine an appropriate name
				FString Name;
				FString PackagePath;
				CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

				// Create the factory used to generate the asset
				UDialogueWaveFactory* Factory = NewObject<UDialogueWaveFactory>();
				Factory->InitialSoundWave = Object;
				Factory->InitialSpeakerVoice = DialogueVoice;
				Factory->HasSetInitialTargetVoice = true;

				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UDialogueWave::StaticClass(), Factory);
			}
		}
		else
		{
			TArray<UObject*> ObjectsToSync;

			for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
			{
				USoundWave* Object = (*ObjIt).Get();
				if (Object)
				{
					FString Name;
					FString PackageName;
					CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

					// Create the factory used to generate the asset
					UDialogueWaveFactory* Factory = NewObject<UDialogueWaveFactory>();
					Factory->InitialSoundWave = Object;
					Factory->InitialSpeakerVoice = DialogueVoice;
					Factory->HasSetInitialTargetVoice = true;

					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UDialogueWave::StaticClass(), Factory);

					if (NewAsset)
					{
						ObjectsToSync.Add(NewAsset);
					}
				}
			}

			if (ObjectsToSync.Num() > 0)
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
			}
		}
	}

	void FillVoiceMenu(UToolMenu* InMenu, TArray<TSoftObjectPtr<USoundWave>> InSoftObjects)
	{
		TArray<const UClass*> AllowedClasses;
		AllowedClasses.Add(UDialogueVoice::StaticClass());

		TSharedRef<SWidget> VoicePicker = PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
			FAssetData(),
			false,
			AllowedClasses,
			PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
			FOnShouldFilterAsset(),
			FOnAssetSelected::CreateStatic(&ExecuteCreateDialogueWave, InSoftObjects),
			FSimpleDelegate());

		FToolMenuSection& Section = InMenu->FindOrAddSection("Voice");
		Section.AddEntry(FToolMenuEntry::InitWidget("VoicePickerWidget", VoicePicker, FText::GetEmpty(), false));
	}

	void ExecuteCreateSoundCue(const FToolMenuContext& InContext, TArray<TSoftObjectPtr<USoundWave>> InSoftObjects, bool bCreateCueForEachSoundWave)
	{
		const FString DefaultSuffix = TEXT("_Cue");

		TArray<TWeakObjectPtr<USoundWave>> Objects;

		//Load the selected objects to memory
		for (TSoftObjectPtr<USoundWave>& SoundWave : InSoftObjects)
		{
			if (USoundWave* SoundWaveObject = SoundWave.LoadSynchronous())
			{
				Objects.Add(SoundWaveObject);
			}
		}

		if (Objects.Num() == 1 || !bCreateCueForEachSoundWave)
		{
			USoundWave* Object = Objects.Last().Get();

			if (Object)
			{
				// Determine an appropriate name
				FString Name;
				FString PackagePath;		
				CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

				// Create the factory used to generate the asset
				USoundCueFactoryNew* Factory = NewObject<USoundCueFactoryNew>();
				Factory->InitialSoundWaves = Objects;

				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), USoundCue::StaticClass(), Factory);
			}
		}
		else if (bCreateCueForEachSoundWave)
		{
			TArray<UObject*> ObjectsToSync;

			for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
			{
				USoundWave* Object = (*ObjIt).Get();
				if (Object)
				{
					FString Name;
					FString PackageName;
					CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

					// Create the factory used to generate the asset
					USoundCueFactoryNew* Factory = NewObject<USoundCueFactoryNew>();
					Factory->InitialSoundWaves.Add(Object);

					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), USoundCue::StaticClass(), Factory);

					if (NewAsset)
					{
						ObjectsToSync.Add(NewAsset);
					}
				}
			}

			if (ObjectsToSync.Num() > 0)
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		if (!UToolMenus::IsToolMenuUIEnabled())
		{
			return;
		}

		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
			FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
			check(Section);

			Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					TArray<TSoftObjectPtr<USoundWave>> SoundNodes = Context->GetSelectedAssetSoftObjects<USoundWave>();
					bool bCreateCueForEachSoundWave = true;
				
					if (Context->SelectedAssets.Num() == 1)
					{
						const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateSoundCue", "Create SoundCue");
						const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateCueTooltip", "Creates a SoundCue referencing the selected SoundWave.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");
				
						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCue, SoundNodes, bCreateCueForEachSoundWave);
						InSection.AddMenuEntry("SoundWave_CreateCue", Label, ToolTip, Icon, UIAction);
					}
					else
					{
						bCreateCueForEachSoundWave = false;
				
						const TAttribute<FText> Label_Single = LOCTEXT("SoundWave_CreateSingleSoundCue", "Create SoundCue (Single)");
						const TAttribute<FText> ToolTip_Single = LOCTEXT("SoundWave_CreateSingleCueTooltip", "Creates a single SoundCue Referencing the selected SoundWaves.");
						const FSlateIcon Icon_Single = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");
				
						FToolUIAction UIAction_Single;
						UIAction_Single.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCue, SoundNodes, bCreateCueForEachSoundWave);
						InSection.AddMenuEntry("SoundWave_CreateSingleCue", Label_Single, ToolTip_Single, Icon_Single, UIAction_Single);
				
						bCreateCueForEachSoundWave = true;
				
						const TAttribute<FText> Label_Multi = LOCTEXT("SoundWave_CreateMultiCue", "Create SoundCues (Multiple)");
						const TAttribute<FText> ToolTip_Multi = LOCTEXT("SoundWave_CreateMultiCueTooltip", "Creates multiple SoundCues, one referencing each selected SoundWave.");
						const FSlateIcon Icon_Multi = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");
				
						FToolUIAction UIAction_Multi;
						UIAction_Multi.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCue, SoundNodes, bCreateCueForEachSoundWave);
						InSection.AddMenuEntry("SoundWave_CreateMultiCue", Label_Multi, ToolTip_Multi, Icon_Multi, UIAction_Multi);
					}
				
					const TAttribute<FText> Label_Dialogue = LOCTEXT("SoundWave_CreateDialogue", "Create Dialogue");
					const TAttribute<FText> ToolTip_Dialogue = LOCTEXT("SoundWave_CreateDialogueTooltip", "Creates a DialogueWave referencing the selected SoundWave.");
				
					InSection.AddSubMenu(
						"SoundWave_CreateDialogue",
						Label_Dialogue,
						ToolTip_Dialogue,
						FNewToolMenuDelegate::CreateStatic(&FillVoiceMenu, SoundNodes),
						false,
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.DialogueWave")
					);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
