// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_AudioDiffable.h"

#include "SDetailsDiff.h"

void FAssetTypeActions_AudioDiffable::PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision,
                                                   const struct FRevisionInfo& NewRevision) const
{
	if (OldAsset == nullptr && NewAsset == nullptr)
	{
		return;
	}
	
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(OldAsset, NewAsset, OldRevision, NewRevision, GetSupportedClass());
	// allow users to edit NewAsset if it's a local asset
	if (NewAsset && !FPackageName::IsTempPackage(NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(NewAsset);
	}
}
