// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_SoundBase.h"
#include "AudioDeviceManager.h"
#include "Audio/AudioDebug.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundWave.h"
#include "ToolMenus.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Components/AudioComponent.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_SoundBase::GetSupportedClass() const
{
	return USoundBase::StaticClass();
}

bool FAssetTypeActions_SoundBase::AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType)
{
	if (ActivationType == EAssetTypeActivationMethod::Previewed)
	{
		USoundBase* TargetSound = NULL;

		for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			TargetSound = Cast<USoundBase>(*ObjIt);
			if (TargetSound)
			{
				// Only target the first valid sound cue
				break;
			}
		}

		UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
		if (PreviewComp && PreviewComp->IsPlaying())
		{
			// Already previewing a sound, if it is the target cue then stop it, otherwise play the new one
			if (!TargetSound || PreviewComp->Sound == TargetSound)
			{
				StopSound();
			}
			else
			{
				PlaySound(TargetSound);
			}
		}
		else
		{
			// Not already playing, play the target sound cue if it exists
			PlaySound(TargetSound);
		}
		return true;
	}
	return false;
}

void FAssetTypeActions_SoundBase::PlaySound(USoundBase* Sound) const
{
	if ( Sound )
	{
		GEditor->PlayPreviewSound(Sound);
	}
	else
	{
		StopSound();
	}
}

void FAssetTypeActions_SoundBase::StopSound() const
{
	GEditor->ResetPreviewAudioComponent();
}

bool FAssetTypeActions_SoundBase::IsSoundPlaying(USoundBase* Sound) const
{
	UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
	return PreviewComp && PreviewComp->Sound == Sound && PreviewComp->IsPlaying();
}

bool FAssetTypeActions_SoundBase::IsSoundPlaying(const FAssetData& AssetData) const
{
	const UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
	if (PreviewComp && PreviewComp->Sound && PreviewComp->IsPlaying())
	{
		if (PreviewComp->Sound->GetFName() == AssetData.AssetName)
		{
			if (PreviewComp->Sound->GetOutermost()->GetFName() == AssetData.PackageName)
			{
				return true;
			}
		}
	}

	return false;
}

void FAssetTypeActions_SoundBase::ExecutePlaySound(TArray<TWeakObjectPtr<USoundBase>> Objects) const
{
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		USoundBase* Sound = (*ObjIt).Get();
		if (Sound)
		{
			// Only play the first valid sound
			PlaySound(Sound);
			break;
		}
	}
}

void FAssetTypeActions_SoundBase::ExecuteStopSound(TArray<TWeakObjectPtr<USoundBase>> Objects) const
{
	StopSound();
}

TSharedPtr<SWidget> FAssetTypeActions_SoundBase::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	auto OnGetDisplayBrushLambda = [this, AssetData]() -> const FSlateBrush*
	{
		if (IsSoundPlaying(AssetData))
		{
			return FAppStyle::GetBrush("MediaAsset.AssetActions.Stop.Large");
		}

		return FAppStyle::GetBrush("MediaAsset.AssetActions.Play.Large");
	};

	auto OnClickedLambda = [this, AssetData]() -> FReply
	{
		if (IsSoundPlaying(AssetData))
		{
			StopSound();
		}
		else
		{
			// Load and play sound
			PlaySound(Cast<USoundBase>(AssetData.GetAsset()));
		}
		return FReply::Handled();
	};

	auto OnToolTipTextLambda = [this, AssetData]() -> FText
	{
		if (IsSoundPlaying(AssetData))
		{
			return LOCTEXT("Thumbnail_StopSoundToolTip", "Stop selected sound");
		}

		return LOCTEXT("Thumbnail_PlaySoundToolTip", "Play selected sound");
	};

	TSharedPtr<SBox> Box;
	SAssignNew(Box, SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2));

	auto OnGetVisibilityLambda = [this, Box, AssetData]() -> EVisibility
	{
		if (Box.IsValid() && (Box->IsHovered() || IsSoundPlaying(AssetData)))
		{
			return EVisibility::Visible;
		}

		return EVisibility::Hidden;
	};

	TSharedPtr<SButton> Widget;
	SAssignNew(Widget, SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ToolTipText_Lambda(OnToolTipTextLambda)
		.Cursor(EMouseCursor::Default) // The outer widget can specify a DragHand cursor, so we need to override that here
		.ForegroundColor(FSlateColor::UseForeground())
		.IsFocusable(false)
		.OnClicked_Lambda(OnClickedLambda)
		.Visibility_Lambda(OnGetVisibilityLambda)
		[
			SNew(SImage)
			.Image_Lambda(OnGetDisplayBrushLambda)
		];

	Box->SetContent(Widget.ToSharedRef());
	Box->SetVisibility(EVisibility::Visible);

	return Box;
}

#undef LOCTEXT_NAMESPACE
