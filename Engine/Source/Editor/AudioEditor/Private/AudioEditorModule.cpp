// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioEditorModule.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "AudioEditorSettings.h"
#include "ClassTemplateEditorSubsystem.h"
#include "ComponentAssetBroker.h"
#include "Components/SynthComponent.h"
#include "EdGraphUtilities.h"
#include "Factories/ReimportSoundFactory.h"
#include "Factories/SoundFactory.h"
#include "HAL/LowLevelMemTracker.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Sound/AudioSettings.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundClass.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundSubmix.h"
#include "Sound/SoundWave.h"
#include "SoundClassEditor.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "SoundCueGraphConnectionDrawingPolicy.h"
#include "SoundCueGraphNodeFactory.h"
#include "SoundCueEditor.h"
#include "SoundFileIO/SoundFileIO.h"
#include "SoundModulationDestinationLayout.h"
#include "SoundSubmixEditor.h"
#include "SoundSubmixGraph/SoundSubmixGraphSchema.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "SubmixDetailsCustomization.h"
#include "UObject/UObjectIterator.h"
#include "Utils.h"
#include "WidgetBlueprint.h"
#include "AssetTypeActions/AssetDefinition_SoundEffectPreset.h"
const FName AudioEditorAppIdentifier("AudioEditorApp");

DEFINE_LOG_CATEGORY(LogAudioEditor);

class FSlateStyleSet;
struct FGraphPanelPinConnectionFactory;

// Setup icon sizes
static const FVector2D Icon16 = FVector2D(16.0f, 16.0f);
static const FVector2D Icon64 = FVector2D(64.0f, 64.0f);

// Preprocessor macro to make defining audio icons simple...

// CLASS_NAME - name of the class to make the icon for
// ICON_NAME - base-name of the icon to use. Not necessarily based off class name
#define SET_AUDIO_ICON(CLASS_NAME, ICON_NAME) \
		AudioStyleSet->Set( *FString::Printf(TEXT("ClassIcon.%s"), TEXT(#CLASS_NAME)), new FSlateImageBrush(FPaths::EngineContentDir() / FString::Printf(TEXT("Editor/Slate/Icons/AssetIcons/%s_16x.png"), TEXT(#ICON_NAME)), Icon16)); \
		AudioStyleSet->Set( *FString::Printf(TEXT("ClassThumbnail.%s"), TEXT(#CLASS_NAME)), new FSlateImageBrush(FPaths::EngineContentDir() / FString::Printf(TEXT("Editor/Slate/Icons/AssetIcons/%s_64x.png"), TEXT(#ICON_NAME)), Icon64));

// Simpler version of SET_AUDIO_ICON, assumes same name of icon png and class name
#define SET_AUDIO_ICON_SIMPLE(CLASS_NAME) SET_AUDIO_ICON(CLASS_NAME, CLASS_NAME)

#define SET_AUDIO_ICON_SVG(CLASS_NAME, ICON_NAME) \
		AudioStyleSet->Set( *FString::Printf(TEXT("ClassIcon.%s"), TEXT(#CLASS_NAME)), new FSlateVectorImageBrush(FPaths::EngineContentDir() / FString::Printf(TEXT("Editor/Slate/Starship/AssetIcons/%s_16.svg"), TEXT(#ICON_NAME)), Icon16)); \
		AudioStyleSet->Set( *FString::Printf(TEXT("ClassThumbnail.%s"), TEXT(#CLASS_NAME)), new FSlateVectorImageBrush(FPaths::EngineContentDir() / FString::Printf(TEXT("Editor/Slate/Starship/AssetIcons/%s_64.svg"), TEXT(#ICON_NAME)), Icon64));

#define SET_AUDIO_ICON_SVG_SIMPLE(CLASS_NAME) SET_AUDIO_ICON_SVG(CLASS_NAME, CLASS_NAME)

UClass* FAudioComponentBroker::GetSupportedAssetClass()
{
	return USoundBase::StaticClass();
}

bool FAudioComponentBroker::AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset)
{
	if (UAudioComponent* AudioComp = Cast<UAudioComponent>(InComponent))
	{
		USoundBase* Sound = Cast<USoundBase>(InAsset);

		if ((Sound != NULL) || (InAsset == NULL))
		{
			AudioComp->SetSound(Sound);
			return true;
		}
	}

	return false;
}

UObject* FAudioComponentBroker::GetAssetFromComponent(UActorComponent* InComponent)
{
	if (UAudioComponent* AudioComp = Cast<UAudioComponent>(InComponent))
	{
		return AudioComp->Sound;
	}
	return NULL;
}

class FAudioEditorModule : public IAudioEditorModule
{
public:
	FAudioEditorModule()
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
		// Create style set for audio asset icons
		AudioStyleSet = MakeShared<FSlateStyleSet>("AudioStyleSet");
	}

	virtual void StartupModule() override
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
		SoundClassExtensibility.Init();
		SoundCueExtensibility.Init();
		SoundSubmixExtensibility.Init();

		// Register the sound cue graph connection policy with the graph editor
		SoundCueGraphConnectionFactory = MakeShared<FSoundCueGraphConnectionDrawingPolicyFactory>();
		FEdGraphUtilities::RegisterVisualPinConnectionFactory(SoundCueGraphConnectionFactory);

		// Register the sound cue graph connection policy with the graph editor
		SoundSubmixGraphConnectionFactory = MakeShared<FSoundSubmixGraphConnectionDrawingPolicyFactory>();
		FEdGraphUtilities::RegisterVisualPinConnectionFactory(SoundSubmixGraphConnectionFactory);

		TSharedPtr<FSoundCueGraphNodeFactory> SoundCueGraphNodeFactory = MakeShared<FSoundCueGraphNodeFactory>();
		FEdGraphUtilities::RegisterVisualNodeFactory(SoundCueGraphNodeFactory);

		// Create reimport handler for sound node waves
		UReimportSoundFactory::StaticClass();

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		
		// Custom Property Layouts
		auto AddCustomProperty = [this, InPropertyModule = &PropertyModule](FName Name, FOnGetPropertyTypeCustomizationInstance InstanceGetter)
		{
			InPropertyModule->RegisterCustomPropertyTypeLayout(Name, InstanceGetter);
			CustomPropertyLayoutNames.Add(Name);
		};
		AddCustomProperty("SoundModulationDestinationSettings", 
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundModulationDestinationLayoutCustomization::MakeInstance));
		AddCustomProperty("SoundModulationDefaultSettings",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundModulationDefaultSettingsLayoutCustomization::MakeInstance));
		AddCustomProperty("SoundModulationDefaultRoutingSettings",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundModulationDefaultRoutingSettingsLayoutCustomization::MakeInstance));

		// Custom Class Layouts
		auto AddCustomClass = [this, InPropertyModule = &PropertyModule](FName Name, FOnGetDetailCustomizationInstance InstanceGetter)
		{
			InPropertyModule->RegisterCustomClassLayout(Name, InstanceGetter);
			CustomClassLayoutNames.Add(Name);
		};

		// Load the module for Audio Extensions so it doesn't rely on AutoDiscoveredSettings which can get unregistered since it is utilized by the following custom classes
		FModuleManager::Get().LoadModule(TEXT("AudioExtensions"));

		AddCustomClass("EndpointSubmix", FOnGetDetailCustomizationInstance::CreateStatic(&FEndpointSubmixDetailsCustomization::MakeInstance));
		AddCustomClass("SoundfieldEndpointSubmix", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundfieldEndpointSubmixDetailsCustomization::MakeInstance));
		AddCustomClass("SoundfieldSubmix", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundfieldSubmixDetailsCustomization::MakeInstance));

		AddCustomClass("AudioEditorSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FAudioOutputDeviceCustomization::MakeInstance));

		SetupIcons();

		// Disable audio component asset broker in restricted mode 
		// to disable CreateBlueprintUsing asset action
		if (!IsRestrictedMode())
		{
			AudioComponentBroker = MakeShareable(new FAudioComponentBroker());
			FComponentAssetBrokerage::RegisterBroker(AudioComponentBroker, UAudioComponent::StaticClass(), true, true);
		}
#if WITH_SNDFILE_IO
		if (!Audio::SoundFileUtils::InitSoundFileIOManager())
		{
			UE_LOGF(LogAudioEditor, Display, "LibSoundFile failed to load. Importing audio will not work correctly.");
		}
#endif // WITH_SNDFILE_IO
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
#if WITH_SNDFILE_IO
		Audio::SoundFileUtils::ShutdownSoundFileIOManager();
#endif // WITH_SNDFILE_IO

		SoundClassExtensibility.Reset();
		SoundCueExtensibility.Reset();
		SoundSubmixExtensibility.Reset();

		if (SoundCueGraphConnectionFactory.IsValid())
		{
			FEdGraphUtilities::UnregisterVisualPinConnectionFactory(SoundCueGraphConnectionFactory);
		}

		if (SoundSubmixGraphConnectionFactory.IsValid())
		{
			FEdGraphUtilities::UnregisterVisualPinConnectionFactory(SoundSubmixGraphConnectionFactory);
		}

		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

			for (FName PropertyName : CustomPropertyLayoutNames)
			{
				PropertyModule.UnregisterCustomPropertyTypeLayout(PropertyName);
			}
			CustomPropertyLayoutNames.Reset();

			for (FName ClassName : CustomClassLayoutNames)
			{
				PropertyModule.UnregisterCustomClassLayout(ClassName);
			}
			CustomClassLayoutNames.Reset();
		}

		if (UObjectInitialized() && !IsRestrictedMode())
		{
			FComponentAssetBrokerage::UnregisterBroker(AudioComponentBroker);
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void RegisterAssetActions() override
	{}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void RegisterAudioMixerAssetActions() override
	{}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void RegisterEffectPresetAssetActions() override
	{
		// Look for any sound effect presets to register
		for (TObjectIterator<UClass> It; It; ++It)
		{
			UClass* ChildClass = *It;
			if (ChildClass->HasAnyClassFlags(CLASS_Abstract))
			{
				continue;
			}

			// Look for submix or source preset classes
			UClass* ParentClass = ChildClass->GetSuperClass();
			if (ParentClass && (ParentClass->IsChildOf(USoundEffectSourcePreset::StaticClass()) || ParentClass->IsChildOf(USoundEffectSubmixPreset::StaticClass())))
			{
				USoundEffectPreset* EffectPreset = ChildClass->GetDefaultObject<USoundEffectPreset>();
				if (!RegisteredActions.Contains(EffectPreset) && EffectPreset->HasAssetActions())
				{
					RegisteredActions.Add(EffectPreset);
					UAssetDefinition_SoundEffectPresetDynamic* Definition = NewObject<UAssetDefinition_SoundEffectPresetDynamic>(
						GetTransientPackage(),
						MakeUniqueObjectName(GetTransientPackage(), UAssetDefinition_SoundEffectPresetDynamic::StaticClass())
					);
					Definition->Initialize(TSubclassOf<USoundEffectPreset>(ChildClass));
					UAssetDefinitionRegistry::Get()->RegisterAssetDefinition(Definition);
				}
			}
		}
	}

	virtual TSharedRef<FAssetEditorToolkit> CreateSoundClassEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, USoundClass* InSoundClass ) override
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
		TSharedRef<FSoundClassEditor> NewSoundClassEditor(new FSoundClassEditor());
		NewSoundClassEditor->InitSoundClassEditor(Mode, InitToolkitHost, InSoundClass);
		return NewSoundClassEditor;
	}

	virtual TSharedRef<FAssetEditorToolkit> CreateSoundSubmixEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, USoundSubmixBase* InSoundSubmix) override
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
		TSharedPtr<FSoundSubmixEditor> NewSubmixEditor = MakeShared<FSoundSubmixEditor>();
		NewSubmixEditor->Init(Mode, InitToolkitHost, InSoundSubmix);
		return StaticCastSharedPtr<FAssetEditorToolkit>(NewSubmixEditor).ToSharedRef();
	}

	virtual TSharedPtr<FExtensibilityManager> GetSoundClassMenuExtensibilityManager() override
	{
		return SoundClassExtensibility.MenuExtensibilityManager;
	}

	virtual TSharedPtr<FExtensibilityManager> GetSoundClassToolBarExtensibilityManager() override
	{
		return SoundClassExtensibility.ToolBarExtensibilityManager;
	}

	virtual TSharedPtr<FExtensibilityManager> GetSoundSubmixMenuExtensibilityManager() override
	{
		return SoundSubmixExtensibility.MenuExtensibilityManager;
	}

	virtual TSharedPtr<FExtensibilityManager> GetSoundSubmixToolBarExtensibilityManager() override
	{
		return SoundSubmixExtensibility.ToolBarExtensibilityManager;
	}

	virtual TSharedRef<ISoundCueEditor> CreateSoundCueEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, USoundCue* SoundCue) override
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
		TSharedRef<FSoundCueEditor> NewSoundCueEditor(new FSoundCueEditor());
		NewSoundCueEditor->InitSoundCueEditor(Mode, InitToolkitHost, SoundCue);
		return NewSoundCueEditor;
	}

	virtual TSharedPtr<FExtensibilityManager> GetSoundCueMenuExtensibilityManager() override
	{
		return SoundCueExtensibility.MenuExtensibilityManager;
	}

	virtual TSharedPtr<FExtensibilityManager> GetSoundCueToolBarExtensibilityManager() override
	{
		return SoundCueExtensibility.MenuExtensibilityManager;
	}

	virtual void ReplaceSoundNodesInGraph(USoundCue* SoundCue, UDialogueWave* DialogueWave, TArray<USoundNode*>& NodesToReplace, const FDialogueContextMapping& ContextMapping) override
	{
		// Replace any sound nodes in the graph.
		TArray<USoundCueGraphNode*> GraphNodesToRemove;
		for (USoundNode* const SoundNode : NodesToReplace)
		{
			// Create the new dialogue wave player.
			USoundNodeDialoguePlayer* DialoguePlayer = SoundCue->ConstructSoundNode<USoundNodeDialoguePlayer>();
			DialoguePlayer->SetDialogueWave(DialogueWave);
			DialoguePlayer->DialogueWaveParameter.Context = ContextMapping.Context;

			// We won't need the newly created graph node as we're about to move the dialogue wave player onto the original node.
			GraphNodesToRemove.Add(CastChecked<USoundCueGraphNode>(DialoguePlayer->GetGraphNode()));

			// Swap out the sound wave player in the graph node with the new dialogue wave player.
			USoundCueGraphNode* SoundGraphNode = CastChecked<USoundCueGraphNode>(SoundNode->GetGraphNode());
			SoundGraphNode->SetSoundNode(DialoguePlayer);
		}

		for (USoundCueGraphNode* const SoundGraphNode : GraphNodesToRemove)
		{
			SoundCue->GetGraph()->RemoveNode(SoundGraphNode);
		}

		// Make sure the cue is updated to match its graph.
		SoundCue->CompileSoundNodesFromGraphNodes();

		for (USoundNode* const SoundNode : NodesToReplace)
		{
			// Remove the old node from the list of available nodes.
			SoundCue->AllNodes.Remove(SoundNode);
		}
		SoundCue->MarkPackageDirty();
	}

	USoundWave* ImportSoundWave(UPackage* const SoundWavePackage, const FString& InSoundWaveAssetName, const FString& InWavFilename) override
	{
		LLM_SCOPE(ELLMTag::AudioMisc);
		USoundFactory* SoundWaveFactory = NewObject<USoundFactory>();

		// Setup sane defaults for importing localized sound waves
		SoundWaveFactory->bAutoCreateCue = false;
		SoundWaveFactory->SuppressImportDialogs();

		return ImportObject<USoundWave>(SoundWavePackage, *InSoundWaveAssetName, RF_Public | RF_Standalone, *InWavFilename, nullptr, SoundWaveFactory);
	}

	bool IsRestrictedMode() const override
	{
		return bIsRestricted;
	}

	void SetRestrictedMode(bool bInRestricted) override
	{
		bIsRestricted = bInRestricted;
	}

	virtual TOptional<FAudioEditorDeviceSettings> GetAudioEditorDeviceSettings() const override
	{
		FAudioEditorDeviceSettings DeviceSettings;
		if (const UAudioEditorSettings* AudioEditorSettings = GetDefault<UAudioEditorSettings>())
		{
			DeviceSettings.bUseSystemDevice = AudioEditorSettings->bUseSystemDevice;
			DeviceSettings.DeviceId = AudioEditorSettings->AudioOutputDeviceId;
		}
		
		return DeviceSettings;
	}

private:

	void SetupIcons()
	{
		SET_AUDIO_ICON_SIMPLE(SoundAttenuation);
		SET_AUDIO_ICON_SVG_SIMPLE(AmbientSound);
		SET_AUDIO_ICON_SIMPLE(SoundClass);
		SET_AUDIO_ICON_SIMPLE(SoundConcurrency);
		SET_AUDIO_ICON_SIMPLE(SoundCue);
		SET_AUDIO_ICON_SIMPLE(SoundMix);
		SET_AUDIO_ICON_SVG_SIMPLE(AudioVolume);
		SET_AUDIO_ICON_SIMPLE(SoundSourceBus);
		SET_AUDIO_ICON_SIMPLE(SoundSubmix);
		SET_AUDIO_ICON_SIMPLE(ReverbEffect);

		SET_AUDIO_ICON(EndpointSubmix, SoundSubmix);
		SET_AUDIO_ICON(SoundfieldEndpointSubmix, SoundSubmix);
		SET_AUDIO_ICON(SoundfieldSubmix, SoundSubmix);

		SET_AUDIO_ICON(SoundEffectSubmixPreset, SubmixEffectPreset);
		SET_AUDIO_ICON(SoundEffectSourcePreset, SourceEffectPreset);
		SET_AUDIO_ICON(SoundEffectSourcePresetChain, SourceEffectPresetChain_1);
		SET_AUDIO_ICON(ModularSynthPresetBank, SoundGenericIcon_2);
		SET_AUDIO_ICON(MonoWaveTableSynthPreset, SoundGenericIcon_2);
		SET_AUDIO_ICON(TimeSynthClip, SoundGenericIcon_2);
		SET_AUDIO_ICON(TimeSynthVolumeGroup, SoundGenericIcon_1);

		FSlateStyleRegistry::RegisterSlateStyle(*AudioStyleSet.Get());
	}

	struct FExtensibilityManagers
	{
		TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
		TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

		void Init()
		{
			MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
			ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();
		}

		void Reset()
		{
			MenuExtensibilityManager.Reset();
			ToolBarExtensibilityManager.Reset();
		}
	};

	TArray<FName> CustomClassLayoutNames;
	TArray<FName> CustomPropertyLayoutNames;
	TMap<TSubclassOf<USoundEffectPreset>, UWidgetBlueprint*> EffectPresetWidgets;

	FExtensibilityManagers SoundCueExtensibility;
	FExtensibilityManagers SoundClassExtensibility;
	FExtensibilityManagers SoundSubmixExtensibility;
	TSet<USoundEffectPreset*> RegisteredActions;
	TSharedPtr<FGraphPanelPinConnectionFactory> SoundCueGraphConnectionFactory;
	TSharedPtr<FGraphPanelPinConnectionFactory> SoundSubmixGraphConnectionFactory;
	TSharedPtr<FSlateStyleSet> AudioStyleSet;

	TSharedPtr<IComponentAssetBroker> AudioComponentBroker;

	// Whether or not the editor is in restricted mode
	// (to disable asset actions)
	bool bIsRestricted = false;
};

IMPLEMENT_MODULE( FAudioEditorModule, AudioEditor );
