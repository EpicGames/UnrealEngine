// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioEditorSettings.h"
#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "AudioDeviceManager.h"
#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "DetailsNameWidgetOverrideCustomization.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "SSimpleComboButton.h"
#include "ToolMenus.h"
#include "Widgets/Input/SCheckBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioEditorSettings)

#define LOCTEXT_NAMESPACE "FAudioOutputDevicePropertyCustomization"


TArray<FName> UAudioEditorSettings::GetAllowedMenuPositions()
{
	const UEnum* EnumClass = StaticEnum<EToolMenuInsertType>();
	check(EnumClass);

	auto ToAuthoredFName = [&EnumClass](const EToolMenuInsertType& InType)
	{
		return FName(*EnumClass->GetAuthoredNameStringByValue(static_cast<uint8>(InType)));
	};
	return TArray<FName> { ToAuthoredFName(EToolMenuInsertType::First), ToAuthoredFName(EToolMenuInsertType::Last) };
}

void UAudioEditorSettings::PostInitProperties()
{
	Super::PostInitProperties();

	ApplyAttenuationForAllAudioDevices();
	FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddUObject(this, &UAudioEditorSettings::ApplyAttenuationForAudioDevice);
}

void UAudioEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioEditorSettings, bUseAudioAttenuation))
	{
		ApplyAttenuationForAllAudioDevices();
	}

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioEditorSettings, MenuPosition))
		{
			RebuildSoundContextMenuSections();
		}
	}
}

void UAudioEditorSettings::RebuildSoundContextMenuSections() const
{
	if (UToolMenus::IsToolMenuUIEnabled())
	{
		const UAssetDefinitionRegistry* DefRegistry = UAssetDefinitionRegistry::Get();
		check(DefRegistry);

		TArray<TObjectPtr<UAssetDefinition>> AssetDefinitions = DefRegistry->GetAllAssetDefinitions();
		for (TObjectPtr<UAssetDefinition> AssetDef : AssetDefinitions)
		{
			if (UAssetDefinition_SoundAssetBase* SoundAssetDef = Cast<UAssetDefinition_SoundAssetBase>(AssetDef.Get()))
			{
				SoundAssetDef->RebuildSoundContextMenuSections();
			}
		}
	}
}
void UAudioEditorSettings::SetUseAudioAttenuation(bool bInUseAudioAttenuation) 
{
	bUseAudioAttenuation = bInUseAudioAttenuation;
	SaveConfig();
	ApplyAttenuationForAllAudioDevices();
}

void UAudioEditorSettings::ApplyAttenuationForAllAudioDevices()
{
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		TArray<FAudioDevice*> AudioDevices = AudioDeviceManager->GetAudioDevices();
		for (FAudioDevice* Device : AudioDevices)
		{
			if (Device)
			{
				Device->SetUseAttenuationForNonGameWorlds(bUseAudioAttenuation);
			}
		}
	}
}

void UAudioEditorSettings::ApplyAttenuationForAudioDevice(Audio::FDeviceId InDeviceID)
{
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		FAudioDeviceHandle Device = AudioDeviceManager->GetAudioDevice(InDeviceID);
		if (Device.IsValid())
		{
			Device->SetUseAttenuationForNonGameWorlds(bUseAudioAttenuation);
		}
	}	
}

TSharedRef<IDetailCustomization> FAudioOutputDeviceCustomization::MakeInstance()
{
	return MakeShareable(new FAudioOutputDeviceCustomization);
}

FAudioOutputDeviceCustomization::FAudioOutputDeviceCustomization()
{
	TickDelegate = FTickerDelegate::CreateRaw(this, &FAudioOutputDeviceCustomization::Tick);

	// Only populate the AudioBackends array if the settings will be visible to the user
	if (IsDeviceSettingsEnabled())
	{
		if (const UAudioEditorSettings* AudioEditorSettings = GetDefault<UAudioEditorSettings>())
		{
			AudioMixerPlatforms = AudioEditorSettings->AudioMixerPlatforms;
			
			for (const FAudioPlatform& AudioPlatform : AudioMixerPlatforms)
			{
				AudioBackends.Add(MakeShared<FString>(AudioPlatform.DisplayName));
			}
		}
	}
	
	CurrentBackendName = GetCurrentBackendName();
}

FAudioOutputDeviceCustomization::~FAudioOutputDeviceCustomization()
{
	if (TickDelegateHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickDelegateHandle);
	}
}

bool FAudioOutputDeviceCustomization::Tick(float DeltaTime)
{
	bool bUseSystemDefault = false;
	if (UseSystemDevicePropertyHandle->GetValue(bUseSystemDefault) == FPropertyAccess::Result::Success)
	{
		// The system default device can change when the user selects a new
		// device via the OS settings. If the details are open, periodically check
		// if the device name needs to be updated.
		if (bUseSystemDefault)
		{
			FString NewDeviceName = GetCurrentAudioMixerDeviceName();
			if (NewDeviceName != CurrentDeviceName)
			{
				CurrentDeviceName = NewDeviceName;
				if (DeviceListComboButton.IsValid())
				{
					DeviceListComboButton->Invalidate(EInvalidateWidgetReason::Paint);
				}
			}
		}
	}
	
	return true;
}

void FAudioOutputDeviceCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace Audio;

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("AudioOutputDevice"));

	// Hide the entire category on platforms where this functionality is not enabled
	if (!IsDeviceSettingsEnabled())
	{
		Category.SetCategoryVisibility(false);
		return;
	}
	
	const IDetailLayoutBuilder& DetailLayout = Category.GetParentLayout();
	// Add the Platform Audio API menu on supported platforms
	AddPlatformRow(Category, DetailLayout);
	
	UseSystemDevicePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAudioEditorSettings, bUseSystemDevice));

	// Create check box which indicates system default audio device should be used (disables device combo button)
	if (UseSystemDevicePropertyHandle.IsValid())
	{
		IDetailPropertyRow& UseSystemPropertyRow = Category.AddProperty(UseSystemDevicePropertyHandle);
		
		UseSystemPropertyRow.CustomWidget()
		.NameContent()
		[
			UseSystemDevicePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]() -> ECheckBoxState
			{
				bool bUseSystemDefault;
				UseSystemDevicePropertyHandle->GetValue(bUseSystemDefault);

				return bUseSystemDefault ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) -> void
			{
				using namespace Audio;

				bool bUseSystemDefault = (NewState == ECheckBoxState::Checked);
				UseSystemDevicePropertyHandle->SetValue(bUseSystemDefault);
				SetIsListeningForDeviceEvents(bUseSystemDefault);

				TOptional<FAudioPlatformDeviceInfo> DefaultDeviceInfo = FindDefaultOutputDevice();
				if (DefaultDeviceInfo.IsSet())
				{
					if (bUseSystemDefault)
					{
						// This avoids a UI thrash due to the asynchronous nature of device swaps. It will
						// take some time for the swap to complete so we set the UI name to that of the current
						// active device here (which is not the same for aggregate devices).
						CurrentDeviceName = GetCurrentAudioMixerDeviceName();
						
						// Swap to current system default device
						RequestDeviceSwap(DefaultDeviceInfo->DeviceId);
					}
					else
					{
						// If we were using the system default device, and now we're not, lookup the current
						// system default device and set that as the current device.
						AudioDeviceIdPropertyHandle->SetValue(DefaultDeviceInfo->DeviceId);
						CurrentDeviceName = DefaultDeviceInfo->Name;
					}
				}
			})
		];
	}
	
	AudioDeviceIdPropertyHandle = DetailLayout.GetProperty(TEXT("AudioOutputDeviceId"));
	if (AudioDeviceIdPropertyHandle.IsValid())
	{
		IDetailPropertyRow& AudioDevicePropertyRow = Category.AddProperty(AudioDeviceIdPropertyHandle);

		bool bUseSystemDefault;
		UseSystemDevicePropertyHandle->GetValue(bUseSystemDefault);

		// Populate initial value of CurrentDeviceName based depending on the current settings
		if (bUseSystemDefault)
		{
			CurrentDeviceName = GetCurrentAudioMixerDeviceName();
		}
		else
		{
			FString CurrentDeviceId;
			AudioDeviceIdPropertyHandle->GetValue(CurrentDeviceId);
			CurrentDeviceName = GetDeviceNameForDeviceId(CurrentDeviceId);
		}

		AudioDevicePropertyRow.CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Available Audio Output Devices")))
			.ToolTipText(LOCTEXT("AudioOutputDeviceMenuNameToolTip", "Available Audio Output Devices"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SSimpleComboButton)
			.OnGetMenuContent(this, &FAudioOutputDeviceCustomization::OnGenerateDeviceMenu)
			.ToolTipText(LOCTEXT("AudioOutputDeviceMenuValueToolTip", "Available Audio Output Devices"))
			.HasDownArrow(true)
			.UsesSmallText(true)
			.IsEnabled_Lambda([this]()
			{
				bool bUseDefaultValue;
				UseSystemDevicePropertyHandle->GetValue(bUseDefaultValue);
				return !bUseDefaultValue;
			})
			.Text_Lambda([this]()
			{
				return FText::AsCultureInvariant(CurrentDeviceName);
			})
		];

		TSharedPtr<SWidget> NameWidget;
		AudioDevicePropertyRow.GetDefaultWidgets(NameWidget, DeviceListComboButton);
	}
	
	constexpr float TickDelay = 0.1f; // 100ms delay between ticks
	if (!TickDelegateHandle.IsValid())
	{
		TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate, TickDelay);
	}
}

void FAudioOutputDeviceCustomization::AddPlatformRow(IDetailCategoryBuilder& Category, const IDetailLayoutBuilder& DetailLayout)
{
	AudioPlatformPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAudioEditorSettings, AudioMixerModuleName));
	if (AudioPlatformPropertyHandle.IsValid())
	{
		IDetailPropertyRow& AudioPlatformPropertyRow = Category.AddProperty(AudioPlatformPropertyHandle);
		
		AudioPlatformPropertyRow.CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Platform Audio API")))
			.ToolTipText(LOCTEXT("AudioPlatformMenuNameToolTip", "Available Audio Platform API's. Requires restart to take effect."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SSearchableComboBox)
			.SearchVisibility(EVisibility::Collapsed)
			.InitiallySelectedItem(CurrentBackendName)
			.ToolTipText(LOCTEXT("AudioPlatformMenuValueToolTip", "Available Audio Platform API's. Requires restart to take effect."))
			.OptionsSource(&AudioBackends)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
			{
				return SNew(STextBlock)
						.Text(FText::FromString(*InItem))
						.Font(IDetailLayoutBuilder::GetDetailFont());
			})
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NewChoice, ESelectInfo::Type SelectType)
			{
				SetCurrentBackendName(*NewChoice);
			})
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return FText::FromString(*CurrentBackendName);
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
}

bool FAudioOutputDeviceCustomization::IsDeviceSettingsEnabled()
{
#if ENABLE_AUDIO_DEVICE_EDITOR_SETTINGS
	return true;
#else
	return false;
#endif
}

TArray<Audio::FAudioPlatformDeviceInfo> FAudioOutputDeviceCustomization::GetAvailableAudioOutputDevices()
{
	using namespace Audio;

	TArray<FAudioPlatformDeviceInfo> OutputDevices;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* MixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				if (IAudioPlatformDeviceInfoCache* DeviceInfoCache = MixerPlatform->GetDeviceInfoCache())
				{
					OutputDevices = DeviceInfoCache->GetAllActiveOutputDevices();
				}
				else 
				{
					uint32 NumOutputDevices = 0;
					MixerPlatform->GetNumOutputDevices(NumOutputDevices);
					OutputDevices.Reserve(NumOutputDevices);
					FAudioPlatformDeviceInfo CurrentOutputDevice = MixerPlatform->GetPlatformDeviceInfo();

					for (uint32 i = 0; i < NumOutputDevices; ++i)
					{
						FAudioPlatformDeviceInfo DeviceInfo;
						MixerPlatform->GetOutputDeviceInfo(i, DeviceInfo);
						
						OutputDevices.Emplace(MoveTemp(DeviceInfo));
					}
				}
			}
		}
	}

	return OutputDevices;
}

TOptional<Audio::FAudioPlatformDeviceInfo> FAudioOutputDeviceCustomization::FindDefaultOutputDevice()
{
	using namespace Audio;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* MixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				if (IAudioPlatformDeviceInfoCache* DeviceInfoCache = MixerPlatform->GetDeviceInfoCache())
				{
					return DeviceInfoCache->FindDefaultOutputDevice();
				}
			}
		}
	}

	return{};
}

Audio::FAudioPlatformDeviceInfo FAudioOutputDeviceCustomization::GetDeviceInfo(const FString& InDeviceId)
{
	using namespace Audio;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				uint32 NumOutputDevices = 0;
				AudioMixerPlatform->GetNumOutputDevices(NumOutputDevices);

				for (uint32 i = 0; i < NumOutputDevices; ++i)
				{
					FAudioPlatformDeviceInfo DeviceInfo;
					AudioMixerPlatform->GetOutputDeviceInfo(i, DeviceInfo);

					if (DeviceInfo.DeviceId == InDeviceId)
					{
						return DeviceInfo;
					}
				}
			}
		}
	}

	return {};
}

FString FAudioOutputDeviceCustomization::GetDeviceNameForDeviceId(const FString& InDeviceId)
{
	using namespace Audio;

	FAudioPlatformDeviceInfo DeviceInfo = GetDeviceInfo(InDeviceId);
	return DeviceInfo.Name;
}

FString FAudioOutputDeviceCustomization::GetCurrentAudioMixerDeviceName()
{
	using namespace Audio;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				return AudioMixerPlatform->GetCurrentDeviceName();
			}
		}
	}

	return {};
}

void FAudioOutputDeviceCustomization::SetIsListeningForDeviceEvents(bool bInListeningForDeviceEvents)
{
	using namespace Audio;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				AudioMixerPlatform->SetIsListeningForDeviceEvents(bInListeningForDeviceEvents);
			}
		}
	}
}

void FAudioOutputDeviceCustomization::RequestDeviceSwap(const FString& InDeviceId)
{
	using namespace Audio;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				AudioMixerPlatform->RequestDeviceSwap(InDeviceId, /* force */true, TEXT("FAudioOutputDeviceCustomization::RequestDeviceSwap"));
			}
		}
	}
}

bool FAudioOutputDeviceCustomization::IsAggregateHardwareDeviceId(const FString& InDeviceId)
{
	using namespace Audio;

	if (GEditor)
	{
		if (FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(GEditor->GetEditorWorldContext().World()))
		{
			if (Audio::IAudioMixerPlatformInterface* AudioMixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
			{
				if (IAudioPlatformDeviceInfoCache* DeviceInfoCache = AudioMixerPlatform->GetDeviceInfoCache())
				{
					return DeviceInfoCache->IsAggregateHardwareDeviceId(*InDeviceId);
				}
			}
		}
	}

	return false;
}

TSharedRef<SWidget> FAudioOutputDeviceCustomization::OnGenerateDeviceMenu()
{
	using namespace Audio;

	FMenuBuilder MenuBuilder(true, nullptr, nullptr);

	TArray<FAudioPlatformDeviceInfo> OutputDevices = GetAvailableAudioOutputDevices();
	TArray<FAudioPlatformDeviceInfo> AggregateDevices;
	TArray<FAudioPlatformDeviceInfo> NonAggregateDevices;

	for (const FAudioPlatformDeviceInfo& DeviceInfo : OutputDevices)
	{
		if (IsAggregateHardwareDeviceId(DeviceInfo.DeviceId))
		{
			AggregateDevices.Add(DeviceInfo);
		}
		else
		{
			NonAggregateDevices.Add(DeviceInfo);
		}
	}

	if (AggregateDevices.Num() > 0)
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("AggregateDeviceMenuSection", "Aggregate Audio Output Devices"));
		{
			for (const FAudioPlatformDeviceInfo& DeviceInfo : AggregateDevices)
			{
				if (!DeviceInfo.Name.IsEmpty())
				{
					MenuBuilder.AddMenuEntry(
						FText::FromString(DeviceInfo.Name),
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(this, &FAudioOutputDeviceCustomization::MenuItemDeviceSelected, DeviceInfo)
						)
					);
				}
			}
		}
		MenuBuilder.EndSection();
	}
	
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AudioOutputDeviceMenuSection", "Audio Output Devices"));
	{
		for (const FAudioPlatformDeviceInfo& DeviceInfo : NonAggregateDevices)
		{
			if (!DeviceInfo.Name.IsEmpty())
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(DeviceInfo.Name),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAudioOutputDeviceCustomization::MenuItemDeviceSelected, DeviceInfo)
					)
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAudioOutputDeviceCustomization::MenuItemDeviceSelected(Audio::FAudioPlatformDeviceInfo InDeviceInfo)
{
	CurrentDeviceName = InDeviceInfo.Name;
	AudioDeviceIdPropertyHandle->SetValue(InDeviceInfo.DeviceId);

	// User has changed device to be used by the editor, request a device swap
	RequestDeviceSwap(InDeviceInfo.DeviceId);
}

TSharedPtr<FString> FAudioOutputDeviceCustomization::GetCurrentBackendName() const
{
	FString DefaultAudioPlatform;
	FString SectionName = UAudioEditorSettings::StaticClass()->GetPathName();
	
	// Check to see if the editor pref has been set. If not, fall back to engine setting.
	const bool bFoundModuleName = GConfig->GetString(*SectionName, TEXT("AudioMixerModuleName"), DefaultAudioPlatform, GEditorSettingsIni);
	if (!bFoundModuleName || DefaultAudioPlatform.IsEmpty())
	{
		GConfig->GetString(TEXT("Audio"), TEXT("AudioMixerModuleName"), DefaultAudioPlatform, GEngineIni);
	}

	const int32 Index = AudioMixerPlatforms.IndexOfByPredicate([DefaultAudioPlatform](const FAudioPlatform& Platform)
	{
		return Platform.ModuleName == DefaultAudioPlatform;
	});
	
	if (AudioBackends.IsValidIndex(Index))
	{
		return AudioBackends[Index];
	}

	return MakeShared<FString>(TEXT("UNKNOWN"));
}

void FAudioOutputDeviceCustomization::SetCurrentBackendName(const FString& InBackendName)
{
	int32 Index = AudioMixerPlatforms.IndexOfByPredicate([InBackendName](const FAudioPlatform& Platform)
	{
		return Platform.DisplayName == InBackendName;
	});
	
	if (AudioMixerPlatforms.IsValidIndex(Index) && AudioPlatformPropertyHandle.IsValid())
	{
		AudioPlatformPropertyHandle->SetValue(AudioMixerPlatforms[Index].ModuleName);
	}
	
	if (AudioBackends.IsValidIndex(Index))
	{
		CurrentBackendName = AudioBackends[Index];
	}
}

#undef LOCTEXT_NAMESPACE
