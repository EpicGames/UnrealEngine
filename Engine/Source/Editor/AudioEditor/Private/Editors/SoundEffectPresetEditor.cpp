// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundEffectPresetEditor.h"

#include "Audio/SoundEffectPresetWidgetInterface.h"
#include "Blueprint/UserWidget.h"
#include "Containers/List.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Sound/SoundEffectPreset.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

class UWorld;
struct FSlateBrush;


#define LOCTEXT_NAMESPACE "SoundEffectPresetEditor"


const FName FSoundEffectPresetEditor::AppIdentifier("SoundEffectPresetEditorApp");
const FName FSoundEffectPresetEditor::PropertiesTabId("SoundEffectPresetEditor_Properties");
const FName FSoundEffectPresetEditor::UserWidgetTabId("SoundEffectPresetEditor_UserWidget");

FSoundEffectPresetEditor::FSoundEffectPresetEditor()
	: SoundEffectPreset(nullptr)
{
}

void FSoundEffectPresetEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SoundEffectPresetEditor", "Sound Effect Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FSoundEffectPresetEditor::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));


	for (int32 i = 0; i < UserWidgets.Num(); i++)
	{
		TStrongObjectPtr<UUserWidget> UserWidget = UserWidgets[i];
		const FString ClassName = SoundEffectPreset->GetClass()->GetName();
		FSlateIcon BPIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.CreateClassBlueprint");

		const FName UserWidgetTabIdIndexed = FName(UserWidgetTabId.ToString() + FString(TEXT("_")) + FString::FromInt(i));
		InTabManager->RegisterTabSpawner(UserWidgetTabIdIndexed, FOnSpawnTab::CreateLambda([this, i](const FSpawnTabArgs& Args) { return SpawnTab_UserWidgetEditor(Args, i); }))
			.SetDisplayName(FText::Format(LOCTEXT("UserEditorTabFormat", "{0} Editor"), FText::FromString(ClassName)))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(BPIcon);
	}
}

void FSoundEffectPresetEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(PropertiesTabId);

	for (int32 i = 0; i < UserWidgets.Num(); i++)
	{
		const FName UserWidgetTabIdIndexed = FName(UserWidgetTabId.ToString() + FString(TEXT("_")) + FString::FromInt(i));
		InTabManager->UnregisterTabSpawner(UserWidgetTabIdIndexed);
	}
}

void FSoundEffectPresetEditor::Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USoundEffectPreset* InPresetToEdit, const TArray<UUserWidget*>& InWidgetBlueprints)
{
	if (!ensure(InPresetToEdit))
	{
		return;
	}

	SoundEffectPreset = TStrongObjectPtr<USoundEffectPreset>(InPresetToEdit);
	InitPresetWidgets(InWidgetBlueprints);
	
	// Support undo/redo
	InPresetToEdit->SetFlags(RF_Transactional);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertiesView = PropertyModule.CreateDetailView(Args);
	PropertiesView->SetObject(InPresetToEdit);

	TSharedRef<FTabManager::FSplitter> TabSplitter = FTabManager::NewSplitter()
		->SetSizeCoefficient(0.9f)
		->SetOrientation(Orient_Horizontal)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.225f)
			->AddTab(PropertiesTabId, ETabState::OpenedTab)
		);

	if (!UserWidgets.IsEmpty())
	{
		TabSplitter->Split
		(
			FTabManager::NewSplitter()
			->SetSizeCoefficient(0.775f)
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.33f)
				->AddTab(UserWidgetTabId, ETabState::OpenedTab)
			)
		);
	}

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_SoundEffectPresetEditor_Layout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split(TabSplitter)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	const bool bToolbarFocusable = false;
	const bool bUseSmallIcons = true;
	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		AppIdentifier,
		StandaloneDefaultLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		InPresetToEdit,
		bToolbarFocusable,
		bUseSmallIcons);
}

bool FSoundEffectPresetEditor::CloseWindow(EAssetEditorCloseReason InCloseReason)
{
	if (FAssetEditorToolkit::CloseWindow(InCloseReason))
	{
		UserWidgets.Reset();
		return true;
	}

	return false;
}

FName FSoundEffectPresetEditor::GetEditorName() const
{
	return "Preset Editor";
}

FName FSoundEffectPresetEditor::GetToolkitFName() const
{
	return FName("SoundEffectPresetEditor");
}

FText FSoundEffectPresetEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Sound Effect Preset Editor");
}

void FSoundEffectPresetEditor::InitPresetWidgets(const TArray<UUserWidget*>& InWidgets)
{
	if (!SoundEffectPreset)
	{
		return;
	}

	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		for (UUserWidget* Widget : InWidgets)
		{
			if (Widget)
			{
				UserWidgets.Add(TStrongObjectPtr<UUserWidget>(Widget));
				ISoundEffectPresetWidgetInterface::Execute_OnConstructed(Widget, SoundEffectPreset.Get());
			}
		}
	}
}

FString FSoundEffectPresetEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "SoundEffect ").ToString();
}

FLinearColor FSoundEffectPresetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

EOrientation FSoundEffectPresetEditor::GetSnapLabelOrientation() const
{
	return FMultiBoxSettings::UseSmallToolBarIcons.Get()
		? EOrientation::Orient_Horizontal
		: EOrientation::Orient_Vertical;
}

void FSoundEffectPresetEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (SoundEffectPreset)
	{
		for (TStrongObjectPtr<UUserWidget>& UserWidget : UserWidgets)
		{
			const FName PropertyName = PropertyThatChanged->GetFName();
			ISoundEffectPresetWidgetInterface::Execute_OnPropertyChanged(UserWidget.Get(), SoundEffectPreset.Get(), PropertyName);
		}
	}
}

void FSoundEffectPresetEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	if (SoundEffectPreset)
	{
		for (TStrongObjectPtr<UUserWidget>& UserWidget : UserWidgets)
		{
			auto Node = PropertyThatChanged->GetHead();
			while(Node)
			{
				if (FProperty* Property = Node->GetValue())
				{
					const FName PropertyName = Property->GetFName();
					ISoundEffectPresetWidgetInterface::Execute_OnPropertyChanged(UserWidget.Get(), SoundEffectPreset.Get(), PropertyName);
				}
				Node = Node->GetNextNode();
			}
		}
	}
}

TSharedRef<SDockTab> FSoundEffectPresetEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("SoundSoundEffectDetailsTitle", "Details"))
		[
			PropertiesView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FSoundEffectPresetEditor::SpawnTab_UserWidgetEditor(const FSpawnTabArgs& Args, int32 WidgetIndex)
{
	FName IconBrushName = ISoundEffectPresetWidgetInterface::Execute_GetIconBrushName(UserWidgets[WidgetIndex].Get());
	if (IconBrushName == FName())
	{
		IconBrushName = "GenericEditor.Tabs.Properties";
	}

	const FSlateBrush* IconBrush = FAppStyle::GetBrush(IconBrushName);

	FText Label = FText::FromString(SoundEffectPreset->GetName());
	if (UserWidgets.Num() < WidgetIndex)
	{
		TSharedPtr<SDockTab> NewTab = SNew(SDockTab)
			.Label(Label)
			.TabColorScale(GetTabColorScale())
			[
				SNew(STextBlock)
					.Text(LOCTEXT("InvalidPresetEditor", "No editor available for SoundEffectPreset.  Widget Blueprint not found."))
			];
		NewTab->SetTabIcon(IconBrush);
		return NewTab.ToSharedRef();
	}

	const FText CustomLabel = ISoundEffectPresetWidgetInterface::Execute_GetEditorName(UserWidgets[WidgetIndex].Get());
	if (!CustomLabel.IsEmpty())
	{
		Label = CustomLabel;
	}

	TSharedPtr<SDockTab> NewTab = SNew(SDockTab)
		.Label(Label)
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				UserWidgets[WidgetIndex]->TakeWidget()
			]
		];
	NewTab->SetTabIcon(IconBrush);
	return NewTab.ToSharedRef();
}

void FSoundEffectPresetEditor::PostUndo(bool bSuccess)
{
}

void FSoundEffectPresetEditor::PostRedo(bool bSuccess)
{
}
#undef LOCTEXT_NAMESPACE
