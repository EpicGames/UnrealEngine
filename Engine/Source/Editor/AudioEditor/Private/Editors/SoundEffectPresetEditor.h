// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EditorUndoClient.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/StrongObjectPtr.h"

class FEditPropertyChain;
class FProperty;
class FSpawnTabArgs;
class FTabManager;
class IDetailsView;
class IToolkitHost;
class SDockTab;
class USoundEffectPreset;
class UUserWidget;
struct FPropertyChangedEvent;


class FSoundEffectPresetEditor : public FAssetEditorToolkit, public FNotifyHook, public FEditorUndoClient
{
public:
	FSoundEffectPresetEditor();
	virtual ~FSoundEffectPresetEditor() = default;

	void Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USoundEffectPreset* PresetToEdit, const TArray<UUserWidget*>& InWidgetBlueprints);

	/** FAssetEditorToolkit interface */
	virtual bool CloseWindow(EAssetEditorCloseReason InCloseReason) override;
	virtual FName GetEditorName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

	/** FNotifyHook interface */
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;
protected:
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:
	/** Initializes all preset user widgets. */
	void InitPresetWidgets(const TArray<UUserWidget*>& InWidgets);

	/**	Spawns the tab allowing for editing/viewing the blueprint widget for the associated SoundEffectPreset */
	TSharedRef<SDockTab> SpawnTab_UserWidgetEditor(const FSpawnTabArgs& Args, int32 WidgetIndex);

	/**	Spawns the tab allowing for editing/viewing details panel */
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/** Get the orientation for the snap value controls. */
	EOrientation GetSnapLabelOrientation() const;

	/** Properties tab */
	TSharedPtr<IDetailsView> PropertiesView;

	/** Settings Editor App Identifier */
	static const FName AppIdentifier;

	TStrongObjectPtr<USoundEffectPreset> SoundEffectPreset;
	TArray<TStrongObjectPtr<UUserWidget>> UserWidgets;

	/** Tab Ids */
	static const FName PropertiesTabId;
	static const FName UserWidgetTabId;
};
