// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/AudioBusFactory.h"

#include "Sound/AudioBus.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioBusFactory)

class FFeedbackContext;
class UClass;
class UObject;

UAudioBusFactory::UAudioBusFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioBus::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UAudioBusFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAudioBus* AudioBus = NewObject<UAudioBus>(InParent, InName, Flags);

	return AudioBus;
}

bool UAudioBusFactory::CanCreateNew() const
{
	return true;
}
