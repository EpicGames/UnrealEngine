// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DialogueVoiceFactory.h"

#include "Sound/DialogueVoice.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DialogueVoiceFactory)

class FFeedbackContext;
class UClass;

UDialogueVoiceFactory::UDialogueVoiceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDialogueVoice::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UDialogueVoiceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UObject>(InParent, Class, Name, Flags);
}
