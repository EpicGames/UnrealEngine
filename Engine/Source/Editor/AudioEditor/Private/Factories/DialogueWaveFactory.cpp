// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DialogueWaveFactory.h"

#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundWave.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DialogueWaveFactory)

class FFeedbackContext;
class UClass;
class UObject;

UDialogueWaveFactory::UDialogueWaveFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDialogueWave::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UDialogueWaveFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class == SupportedClass);
	UDialogueWave* DialogueWave = NewObject<UDialogueWave>(InParent, Name, Flags);

	if (InitialSoundWave)
	{
		DialogueWave->SpokenText = InitialSoundWave->SpokenText_DEPRECATED;
		DialogueWave->bMature = InitialSoundWave->bMature;
	}

	DialogueWave->UpdateContext(DialogueWave->ContextMappings[0], InitialSoundWave, InitialSpeakerVoice, InitialTargetVoices);

	return DialogueWave;
}
