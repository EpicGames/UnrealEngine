// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ReimportSoundFactory.h"
#include "Sound/SoundWave.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "AudioEditorModule.h"
#include "HAL/FileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReimportSoundFactory)


UReimportSoundFactory::UReimportSoundFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundWave::StaticClass();
	Formats.Add(TEXT("wav;Wave audio file"));

#if WITH_SNDFILE_IO
	Formats.Add(TEXT("aif;Audio Interchange File"));
	Formats.Add(TEXT("aiff;Audio Interchange File Format"));
	Formats.Add(TEXT("ogg;OGG Vorbis bitstream format"));
	Formats.Add(TEXT("flac;Free Lossless Audio Codec"));
	Formats.Add(TEXT("opus;OGG OPUS bitstream format"));
	Formats.Add(TEXT("mp3;MPEG Layer 3 Audio"));
#endif // WITH_SNDFILE_IO

	OverwriteOtherAssetTypes = -1;

	bCreateNew = false;
	bAutoCreateCue = false;
	bIncludeAttenuationNode = false;
	bIncludeModulatorNode = false;
	bIncludeLoopingNode = false;
	CueVolume = 0.75f;
	ImportPriority = 1;
}

bool UReimportSoundFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (USoundWave* SoundWave = Cast<USoundWave>(Obj))
	{
		SoundWave->AssetImportData->ExtractFilenames(OutFilenames);

		if (OutFilenames.Num() > 0 && !PreferredReimportPath.IsEmpty() && FPaths::GetExtension(PreferredReimportPath) != FPaths::GetExtension(*OutFilenames[0]))
		{
			if (OverwriteOtherAssetTypes < 0)
			{
				EAppReturnType::Type ReturnValue = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(
					NSLOCTEXT("ReimportSoundFactory", "ReImportOverwriteWarning", "You are attempting to re-import over existing sound(s) that was/were previously imported from a different source extension(s)/format(s).  Would you like to use the new extension(s)/format(s) instead?"),
					FText::FromName(*SoundWave->GetName())));

				OverwriteOtherAssetTypes = ReturnValue == EAppReturnType::Yes ? 1 : 0;
			}
			return OverwriteOtherAssetTypes != 0;
		}

		return true;
	}
	return false;
}

void UReimportSoundFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	USoundWave* SoundWave = Cast<USoundWave>(Obj);
	if (SoundWave && ensure(NewReimportPaths.Num() == 1))
	{
		SoundWave->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportSoundFactory::Reimport(UObject* Obj)
{
	// Only handle valid sound node waves
	if (!Obj || !Obj->IsA(USoundWave::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	USoundWave* SoundWave = Cast<USoundWave>(Obj);
	check(SoundWave);

	const FString Filename = SoundWave->AssetImportData->GetFirstFilename();
	const FString FileExtension = FPaths::GetExtension(Filename);

#if WITH_SNDFILE_IO
	const bool bIsSupportedExtension = FCString::Stricmp(*FileExtension, TEXT("WAV")) == 0
		|| FCString::Stricmp(*FileExtension, TEXT("AIF")) == 0
		|| FCString::Stricmp(*FileExtension, TEXT("FLAC")) == 0
		|| FCString::Stricmp(*FileExtension, TEXT("OGG")) == 0
		|| FCString::Stricmp(*FileExtension, TEXT("OPUS")) == 0
		|| FCString::Stricmp(*FileExtension, TEXT("MP3")) == 0;
#else
	const bool bIsSupportedExtension = FCString::Stricmp(*FileExtension, TEXT("WAV")) == 0;
#endif //WITH_SNDFILE_IO

	// Only handle supported extensions
	if (!bIsSupportedExtension)
	{
		return EReimportResult::Failed;
	}

	// If there is no file path provided, can't reimport from source
	if (!Filename.Len())
	{
		// Since this is a new system most sound node waves don't have paths, so logging has been commented out
		//UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: sound node wave resource does not have path stored.");
		return EReimportResult::Failed;
	}

	UE_LOGF(LogAudioEditor, Log, "Performing atomic reimport of [%ls]", *Filename);

	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*Filename) == INDEX_NONE)
	{
		UE_LOGF(LogAudioEditor, Warning, "-- cannot reimport: source file cannot be found.");
		return EReimportResult::Failed;
	}

	// Always suppress dialogs on re-import
	SuppressImportDialogs();

	bool OutCanceled = false;
	if (!ImportObject(SoundWave->GetClass(), SoundWave->GetOuter(), *SoundWave->GetName(), RF_Public | RF_Standalone, Filename, nullptr, OutCanceled))
	{
		if (OutCanceled)
		{
			UE_LOGF(LogAudioEditor, Warning, "-- import canceled");
			return EReimportResult::Cancelled;
		}

		UE_LOGF(LogAudioEditor, Warning, "-- import failed");
		return EReimportResult::Failed;
	}

	UE_LOGF(LogAudioEditor, Log, "-- imported successfully");

	SoundWave->AssetImportData->Update(Filename);
	SoundWave->InvalidateCompressedData();
	SoundWave->FreeResources();
	SoundWave->UpdatePlatformData();
	SoundWave->MarkPackageDirty();
	SoundWave->SetRedrawThumbnail(true);

	return EReimportResult::Succeeded;
}

int32 UReimportSoundFactory::GetPriority() const
{
	return ImportPriority;
}

void UReimportSoundFactory::CleanUp()
{
	OverwriteOtherAssetTypes = -1;
}
