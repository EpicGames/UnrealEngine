// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ReverbEffectFactory.h"

#include "Sound/ReverbEffect.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReverbEffectFactory)

class FFeedbackContext;
class UClass;
class UObject;

UReverbEffectFactory::UReverbEffectFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UReverbEffect::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UReverbEffectFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UReverbEffect* ReverbEffect = NewObject<UReverbEffect>(InParent, InName, Flags);

	return ReverbEffect;
}
