// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundAttenuationFactory.h"

#include "Sound/SoundAttenuation.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundAttenuationFactory)

class FFeedbackContext;
class UClass;
class UObject;

USoundAttenuationFactory::USoundAttenuationFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundAttenuation::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundAttenuationFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<USoundAttenuation>(InParent, Name, Flags);
}


