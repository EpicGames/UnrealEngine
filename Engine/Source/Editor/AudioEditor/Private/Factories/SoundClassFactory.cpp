// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundClassFactory.h"

#include "AudioDeviceManager.h"
#include "Engine/Engine.h"
#include "Sound/SoundClass.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundClassFactory)

class FFeedbackContext;
class UClass;
class UObject;

USoundClassFactory::USoundClassFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundClass::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundClassFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundClass* SoundClass = NewObject<USoundClass>(InParent, InName, Flags);

	class FAudioDeviceManager* AudioDeviceManager = GEngine ? GEngine->GetAudioDeviceManager() : nullptr;
	if (AudioDeviceManager)
	{
		AudioDeviceManager->InitSoundClasses();
	}

	return(SoundClass);
}
