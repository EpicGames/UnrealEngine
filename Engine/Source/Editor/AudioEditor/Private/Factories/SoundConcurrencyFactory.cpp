// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundConcurrencyFactory.h"

#include "Sound/SoundConcurrency.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundConcurrencyFactory)

class FFeedbackContext;
class UClass;
class UObject;

USoundConcurrencyFactory::USoundConcurrencyFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundConcurrency::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundConcurrencyFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundConcurrency* NewConcurrency = NewObject<USoundConcurrency>(InParent, Name, Flags);

	// bEnableMaxCountPlatformScaling is set to false for back compat, but most use cases will call for this being true.
	NewConcurrency->Concurrency.SetEnableMaxCountPlatformScaling(true);
	return NewConcurrency;
}
