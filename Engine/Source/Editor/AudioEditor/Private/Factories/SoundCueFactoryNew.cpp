// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundCueFactoryNew.h"

#include "Audio.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/PlatformCrt.h"
#include "Sound/DialogueTypes.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundNodeRandom.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundWave.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "SoundCueGraph/SoundCueGraphSchema.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueFactoryNew)

class FFeedbackContext;
class UClass;

USoundCueFactoryNew::USoundCueFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = USoundCue::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundCueFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundCue* SoundCue = NewObject<USoundCue>(InParent, Name, Flags);

	constexpr int32 InitialPosX = -200;
	constexpr int32 InitialPosY = 0;

	TArray<TWeakObjectPtr<UObject>> SoundObjects;
	SoundObjects.Append(InitialSoundWaves);
	SoundObjects.Append(InitialDialogueWaves);

	if (SoundObjects.Num() > 0)
	{
		if (SoundObjects.Num() == 1)
		{
			USoundNode* PlayerNode = USoundCueFactoryNew::CreateSoundPlayerNode(SoundCue, SoundObjects[0].Get(), InitialPosX, InitialPosY);
			SoundCue->FirstNode = PlayerNode;
			SoundCue->LinkGraphNodesFromSoundNodes();
		}
		else
		{
			int32 PosX = InitialPosX;
			int32 PosY = InitialPosY;

			USoundNodeRandom* RandomNode = USoundCueFactoryNew::InsertRandomNode(SoundCue, PosX, PosY);
			PosX -= 300;

			TArray<USoundNode*> PlayerNodes;
			for (TWeakObjectPtr<UObject> SoundObject : SoundObjects)
			{
				if (USoundNode* PlayerNode = USoundCueFactoryNew::CreateSoundPlayerNode(SoundCue, SoundObject.Get(), PosX, PosY))
				{
					const int32 ChildNodeIndex = RandomNode->ChildNodes.Num(); 
					if (ChildNodeIndex < RandomNode->GetMaxChildNodes())
					{ 
						RandomNode->InsertChildNode(ChildNodeIndex);
						RandomNode->ChildNodes[ChildNodeIndex] = PlayerNode;
					}

					PlayerNodes.Add(PlayerNode);
					PosY += 100;
				}
			}

			if (SoundObjects.Num() > RandomNode->GetMaxChildNodes())
			{
				UE_LOGF(LogAudio, Warning, "RandomNode is limited to %d input Sound Sources. Excess Sound Sources in SoundCue %ls will be left unconnected.", RandomNode->GetMaxChildNodes(), *Name.ToString());
			}

			if (UEdGraph* SoundCueGraph = SoundCue->GetGraph())
			{
				if (const USoundCueGraphSchema* GraphSchema = Cast<USoundCueGraphSchema>(SoundCueGraph->GetSchema()))
				{
					GraphSchema->TryConnectNodes(PlayerNodes, RandomNode);
				}
			}
		}
	}

	return SoundCue;
}

USoundNodeRandom* USoundCueFactoryNew::InsertRandomNode(USoundCue* SoundCue, int32 NodePosX, int32 NodePosY)
{
	if (SoundCue)
	{
		USoundNodeRandom* RandomNode = SoundCue->ConstructSoundNode<USoundNodeRandom>();
		SoundCue->FirstNode = RandomNode;
		SoundCue->LinkGraphNodesFromSoundNodes();
		RandomNode->GraphNode->NodePosX = NodePosX;
		RandomNode->GraphNode->NodePosY = NodePosY;

		return RandomNode;
	}
	return nullptr;
}

USoundNode* USoundCueFactoryNew::CreateSoundPlayerNode(USoundCue* SoundCue, UObject* SoundObject, int32 NodePosX, int32 NodePosY)
{
	if (!SoundCue || !SoundObject)
	{
		return nullptr;
	}

	if (USoundWave* SoundWave = Cast<USoundWave>(SoundObject))
	{
		if (USoundNodeWavePlayer* WavePlayer = SoundCue->ConstructSoundNode<USoundNodeWavePlayer>())
		{
			WavePlayer->SetSoundWave(SoundWave);
			WavePlayer->GraphNode->NodePosX = NodePosX - CastChecked<USoundCueGraphNode>(WavePlayer->GetGraphNode())->EstimateNodeWidth();
			WavePlayer->GraphNode->NodePosY = NodePosY;
			return WavePlayer;
		}
	}
	else if (UDialogueWave* DialogueWave = Cast<UDialogueWave>(SoundObject))
	{
		if (USoundNodeDialoguePlayer* DialoguePlayer = SoundCue->ConstructSoundNode<USoundNodeDialoguePlayer>())
		{
			DialoguePlayer->SetDialogueWave(DialogueWave);
			DialoguePlayer->GraphNode->NodePosX = NodePosX - CastChecked<USoundCueGraphNode>(DialoguePlayer->GetGraphNode())->EstimateNodeWidth();
			DialoguePlayer->GraphNode->NodePosY = NodePosY;

			if (DialogueWave->ContextMappings.Num() == 1)
			{
				DialoguePlayer->DialogueWaveParameter.Context.Speaker = DialogueWave->ContextMappings[0].Context.Speaker;
				DialoguePlayer->DialogueWaveParameter.Context.Targets = DialogueWave->ContextMappings[0].Context.Targets;
			}

			return DialoguePlayer;
		}
	}

	return nullptr;
}
