// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundFactory.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Audio.h"
#include "AudioAnalytics.h"
#include "AudioEditorPerProjectUserSettings.h"
#include "Components/AudioComponent.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Modules/ModuleManager.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundNodeModulator.h"
#include "Sound/SoundNodeAttenuation.h"
#include "Sound/SoundWave.h"
#include "AudioDeviceManager.h"
#include "AudioDevice.h"
#include "AudioEditorModule.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "EditorFramework/AssetImportData.h"
#include "AudioCompressionSettingsUtils.h"
#include "SoundFileIO/SoundFileIO.h"
#include "Misc/NamePermissionList.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Math/NumericLimits.h"
#include "Widgets/Layout/SUniformGridPanel.h"

// Disable user import

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundFactory)
static int32 EnableUserSoundwaveImportCvar = 1;
FAutoConsoleVariableRef CVarEnableUserSoundwaveImport(
	TEXT("au.EnableUserSoundwaveImport"),
	EnableUserSoundwaveImportCvar,
	TEXT("Enables letting the user import soundwaves in editor.\n")
	TEXT("0: Disabled, 1: Enabled"),
	ECVF_Default);

static float SoundWaveImportLengthLimitInSecondsCVar = -1.f;
FAutoConsoleVariableRef CVarSoundWaveImportLengthLimitInSeconds(
	TEXT("au.SoundWaveImportLengthLimitInSeconds"),
	SoundWaveImportLengthLimitInSecondsCVar,
	TEXT("When set to a value > 0.0f, Soundwaves with durations greater than the value will fail to import.\n")
	TEXT("if the value is < 0.0f, the length will be unlimited"),
	ECVF_Default);


namespace
{

	bool CanImportSoundWaves()
	{
		// disabled via cvar?
		if(EnableUserSoundwaveImportCvar == 0)
		{
			return false;
		}

		IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
		TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ImportAsset);
		if (AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
		{
			if (!AssetClassPermissionList->PassesFilter(USoundWave::StaticClass()->GetPathName()))
			{
				return false;
			}
		}

		return true;
	}

	void InsertSoundNode(USoundCue* SoundCue, UClass* NodeClass, int32 NodeIndex)
	{
		USoundNode* SoundNode = SoundCue->ConstructSoundNode<USoundNode>(NodeClass);

		// If this node allows >0 children but by default has zero - create a connector for starters
		if (SoundNode->GetMaxChildNodes() > 0 && SoundNode->ChildNodes.Num() == 0)
		{
			SoundNode->CreateStartingConnectors();
		}

		SoundNode->GraphNode->NodePosX = -150 * NodeIndex - 100;
		SoundNode->GraphNode->NodePosY = -35;

		// Link the node to the cue.
		SoundNode->ChildNodes[0] = SoundCue->FirstNode;

		// Link the attenuation node to root.
		SoundCue->FirstNode = SoundNode;

		SoundCue->LinkGraphNodesFromSoundNodes();
	}

	void CreateSoundCue(USoundWave* Sound, UObject* InParent, EObjectFlags Flags, bool bIncludeAttenuationNode, bool bIncludeModulatorNode, bool bIncludeLoopingNode, float CueVolume)
	{
		// then first create the actual sound cue
		FString SoundCueName = FString::Printf(TEXT("%s_Cue"), *Sound->GetName());

		// Create sound cue and wave player
		USoundCue* SoundCue = NewObject<USoundCue>(InParent, *SoundCueName, Flags);
		USoundNodeWavePlayer* WavePlayer = SoundCue->ConstructSoundNode<USoundNodeWavePlayer>();

		int32 NodeIndex = (int32)bIncludeAttenuationNode + (int32)bIncludeModulatorNode + (int32)bIncludeLoopingNode;

		WavePlayer->GraphNode->NodePosX = -150 * NodeIndex - 100;
		WavePlayer->GraphNode->NodePosY = -35;

		// Apply the initial volume.
		SoundCue->VolumeMultiplier = CueVolume;

		WavePlayer->SetSoundWave(Sound);
		SoundCue->FirstNode = WavePlayer;
		SoundCue->LinkGraphNodesFromSoundNodes();

		if (bIncludeLoopingNode)
		{
			WavePlayer->bLooping = true;
		}

		if (bIncludeModulatorNode)
		{
			InsertSoundNode(SoundCue, USoundNodeModulator::StaticClass(), --NodeIndex);
		}

		if (bIncludeAttenuationNode)
		{
			InsertSoundNode(SoundCue, USoundNodeAttenuation::StaticClass(), --NodeIndex);
		}

		// Make sure the content browser finds out about this newly-created object.  This is necessary when sound
		// cues are created automatically after creating a sound node wave.  See use of bAutoCreateCue in USoundTTSFactory.
		if ((Flags & (RF_Public | RF_Standalone)) != 0)
		{
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(SoundCue);
		}
	}
} // namespace <>

USoundFactory::USoundFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SuppressImportDialogOptions = ESuppressImportDialog::None;
	TemplateSoundWave = nullptr;

	SupportedClass = USoundWave::StaticClass();
	Formats.Add(TEXT("wav;Wave Audio File"));

#if WITH_SNDFILE_IO
	Formats.Add(TEXT("aif;Audio Interchange File"));
	Formats.Add(TEXT("aiff;Audio Interchange File Format"));
	Formats.Add(TEXT("ogg;OGG Vorbis bitstream format "));
	Formats.Add(TEXT("flac;Free Lossless Audio Codec"));
	Formats.Add(TEXT("opus;OGG OPUS bitstream format"));
	Formats.Add(TEXT("mp3;MPEG Layer 3 Audio"));
#endif // WITH_SNDFILE_IO

	bCreateNew = false;
	bAutoCreateCue = false;
	bIncludeAttenuationNode = false;
	bIncludeModulatorNode = false;
	bIncludeLoopingNode = false;
	CueVolume = 0.75f;
	CuePackageSuffix = TEXT("_Cue");
	bEditorImport = true;
} 

UObject* USoundFactory::FactoryCreateBinary
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		FileType,
	const uint8*&		Buffer,
	const uint8*		BufferEnd,
	FFeedbackContext*	Warn
	)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, FileType);

	{ // Refuse to accept big files. We currently use TArray<> which will fail if we go over an int32.
		const uint64 Size = BufferEnd - Buffer;
		if (!IntFitsIn<int32>(Size))
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("File '%s' is too big (%umb), Max=%umb"), *Name.ToString(), Size>>20, TNumericLimits<int32>::Max()>>20);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}
	}

	UObject* SoundObject = nullptr;

	// First, see if we support this file type in-engine:
	if (FCString::Stricmp(FileType, TEXT("WAV")) == 0)
	{
		SoundObject = CreateObject(Class, InParent, Name, Flags, Context, FileType, Buffer, BufferEnd, Warn);
	}
#if WITH_SNDFILE_IO
	else
	{
		// Read raw audio data
		TArray<uint8> RawAudioData;
		RawAudioData.Empty(BufferEnd - Buffer);
		RawAudioData.AddUninitialized(BufferEnd - Buffer);
		FMemory::Memcpy(RawAudioData.GetData(), Buffer, RawAudioData.Num());

		// Convert audio data to a wav file in memory
		TArray<uint8> RawWaveData;
		if (Audio::SoundFileUtils::ConvertAudioToWav(RawAudioData, RawWaveData))
		{
			const uint8* Ptr = &RawWaveData[0];

			// Perpetuate the setting of the suppression flag to avoid
			// user notification if we attempt to call CreateObject twice
			SoundObject = CreateObject(Class, InParent, Name, Flags, Context, TEXT("WAV"), Ptr, Ptr + RawWaveData.Num(), Warn);
		}
	}
#endif

	if (!SoundObject)
	{
		// Inform user we failed to create the sound wave
		Warn->Logf(ELogVerbosity::Error, TEXT("Failed to import sound wave %s"), *Name.ToString());
	}

	return SoundObject;
}

UObject* USoundFactory::CreateObject
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		FileType,
	const uint8*&		Buffer,
	const uint8*		BufferEnd,
	FFeedbackContext*	Warn
)
{
	if (FCString::Stricmp(FileType, TEXT("WAV")) == 0)
	{
		// create the group name for the cue
		const FString GroupName = InParent->GetFullGroupName(false);
		FString CuePackageName = InParent->GetOutermost()->GetName();
		CuePackageName += CuePackageSuffix;
		if (GroupName.Len() > 0 && GroupName != TEXT("None"))
		{
			CuePackageName += TEXT(".");
			CuePackageName += GroupName;
		}

		// validate the cue's group
		FText Reason;
		const bool bCuePathIsValid = FName(*CuePackageSuffix).IsValidGroupName(Reason);
		const bool bMoveCue = CuePackageSuffix.Len() > 0 && bCuePathIsValid && bAutoCreateCue;
		if (bAutoCreateCue)
		{
			if (!bCuePathIsValid)
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("SoundFactory", "Import Failed", "Import failed for {0}: {1}"), FText::FromString(CuePackageName), Reason));
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
				return nullptr;
			}
		}

		if (!CanImportSoundWaves())
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("SoundFactory", "Soundwave Import Not Allowed", "Soundwave import is not allowed ({0}: {1})"), FText::FromString(CuePackageName), Reason));
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}


		// if we are creating the cue move it when necessary
		UPackage* CuePackage = bMoveCue ? CreatePackage( *CuePackageName) : nullptr;

		// if the sound already exists, remember the user settings
		USoundWave* ExistingSound = FindObject<USoundWave>(InParent, *Name.ToString());

		// A sound that has been deleted in this editor session but not yet garbage-collected is still
		// found by FindObject. Treat such an object as "not existing" so the user is not prompted to
		// overwrite a deleted asset, and so the half-deleted UObject is not silently revived by reuse, causing a crash.
		const bool bFoundDeletedSound = ExistingSound && (!IsValid(ExistingSound) || !ExistingSound->IsAsset());
		if (bFoundDeletedSound)
		{
			ExistingSound = nullptr;
		}

		TArray<UAudioComponent*> ComponentsToRestart;
		FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
		if (AudioDeviceManager && ExistingSound)
		{
			// Will block internally on audio thread completing outstanding commands
			AudioDeviceManager->StopSoundsUsingResource(ExistingSound, &ComponentsToRestart);

			// Resource data is required to exist, if it hasn't been loaded yet,
			// to properly flush compressed data.  This allows the new version
			// to be auditioned in the editor properly.
			if (!ExistingSound->GetResourceData())
			{
				FName RuntimeFormat = ExistingSound->GetRuntimeFormat();
				ExistingSound->InitAudioResource(RuntimeFormat);
			}

			if (ComponentsToRestart.Num() > 0)
			{
				UE_LOGF(LogAudioEditor, Display, "Stopping the following AudioComponents referencing sound being imported");
				for (UAudioComponent* AudioComponent : ComponentsToRestart)
				{
					UE_LOGF(LogAudioEditor, Display, "Component '%ls' Stopped", *AudioComponent->GetName());
					AudioComponent->Stop();
				}
			}
		}

		if (!ExistingSound)
		{
			UpdateTemplate();
		}

		bool bUseExistingSettings = SuppressImportDialogOptions & ESuppressImportDialog::Overwrite;
		if (ExistingSound && !bUseExistingSettings && !GIsAutomationTesting)
		{
			SuppressImportDialogOptions |= ESuppressImportDialog::Overwrite;
			DisplayOverwriteOptionsDialog(FText::Format(
				NSLOCTEXT("SoundFactory", "ImportOverwriteWarning", "You are about to import '{0}' over an existing sound."),
				FText::FromName(Name)));

			switch (OverwriteYesOrNoToAllState)
			{

			case EAppReturnType::Yes:
			case EAppReturnType::YesAll:
			{
				// Overwrite existing settings
				bUseExistingSettings = false;
				break;
			}
			case EAppReturnType::No:
			case EAppReturnType::NoAll:
			{
				// Preserve existing settings
				bUseExistingSettings = true;
				break;
			}
			default:
			{
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
				return nullptr;
			}
			}
		}

		// See if this may be an ambisonics import by checking ambisonics naming convention (ambix)
		FString RootName = Name.GetPlainNameString();
		FString AmbiXTag = RootName.Right(6).ToLower();
		FString FuMaTag = RootName.Right(5).ToLower();

		// check for AmbiX or FuMa tag for the file
		bool bIsAmbiX = (AmbiXTag == TEXT("_ambix"));
		bool bIsFuMa = (FuMaTag == TEXT("_fuma"));

		TArray<uint8> RawWaveData;
		uint32 RawWaveDataBufferSize = BufferEnd - Buffer;
		RawWaveData.Empty(RawWaveDataBufferSize);
		RawWaveData.AddUninitialized(RawWaveDataBufferSize);
		FMemory::Memcpy(RawWaveData.GetData(), Buffer, RawWaveData.Num());

		// Converted buffer if we need it
		TArray<uint8> ConvertedRawWaveData;

		// Read the wave info and make sure we have valid wave data
		FWaveModInfo WaveInfo;
		FString ErrorMessage;
		if (!WaveInfo.ReadWaveInfo(RawWaveData.GetData(), RawWaveData.Num(), &ErrorMessage))
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("Unable to read wave file '%s' - \"%s\""), *Name.ToString(), *ErrorMessage);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		// If we need to change bit depth, or if the format is not something we know, use libsndfile.
		if (*WaveInfo.pBitsPerSample != 16 || !WaveInfo.IsFormatSupported()) 
		{
#if WITH_SNDFILE_IO
			const uint32 OrigNumSamples = Audio::SoundFileUtils::GetNumSamples(RawWaveData);

			// Attempt to convert to 16 bit audio
			if (Audio::SoundFileUtils::ConvertAudioToWav(RawWaveData, ConvertedRawWaveData))
			{
				WaveInfo = FWaveModInfo();				
				if (!WaveInfo.ReadWaveInfo(ConvertedRawWaveData.GetData(), ConvertedRawWaveData.Num(), &ErrorMessage))
				{
					Warn->Logf(ELogVerbosity::Error, TEXT("Failed to convert to 16 bit WAV source on import."));
					GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
					return nullptr;
				}

				// Sanity check that the same number of samples exist in the converted file as the original
				const uint32 ConvertedNumSamples = WaveInfo.GetNumSamples();
				ensure(ConvertedNumSamples == OrigNumSamples);
			}

			// Copy over the data
			Buffer = ConvertedRawWaveData.GetData();
			RawWaveDataBufferSize = ConvertedRawWaveData.Num() * sizeof(uint8);

#else
			WaveInfo.ReportImportFailure();
			Warn->Logf(ELogVerbosity::Error, TEXT("Only 16 bit WAV source files are supported on this editor platform."));
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
#endif
		}

		// Make sure we have 16 bit audio at this point
		check(*WaveInfo.pBitsPerSample == 16);

		// Validate if somebody has used the ambiX or FuMa tag that the ChannelCount is 4 channels
		if ((bIsAmbiX || bIsFuMa) && (int32)*WaveInfo.pChannels != 4)
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("Tried to import ambisonics format file but requires exactly 4 channels: '%s'"), *Name.ToString());
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		// If we're a multi-channel file, we're going to spoof the behavior of the SoundSurroundFactory
		int32 ChannelCount = (int32)*WaveInfo.pChannels;
		check(ChannelCount > 0);

		int32 SizeOfSample = (*WaveInfo.pBitsPerSample) / 8;

		int32 NumSamples = WaveInfo.SampleDataSize / SizeOfSample;
		int32 NumFrames = NumSamples / ChannelCount;

		const bool bLimitingSoundWaveLength = SoundWaveImportLengthLimitInSecondsCVar > 0.0f;
		const float SoundDuration = (float)NumFrames / *WaveInfo.pSamplesPerSec;
		if (bLimitingSoundWaveLength && SoundDuration >= SoundWaveImportLengthLimitInSecondsCVar)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("SoundFactory", "Soundwave is too long to import"
				, "{0} is {1} seconds in duration (this is over the limit of {2} seconds) {3}")
				, FText::FromString(CuePackageName), SoundDuration, SoundWaveImportLengthLimitInSecondsCVar, Reason));

			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		// Use pre-existing sound if it exists and we want to keep settings,
		// otherwise create new sound and import raw data.
		USoundWave* Sound = (bUseExistingSettings && ExistingSound) ? ExistingSound : NewObject<USoundWave>(InParent, Name, Flags, TemplateSoundWave.Get());

		// These get wiped in PostInitProperties by defaults set from Audio Settings,
		// so set back to template in this specialized case
		if (TemplateSoundWave.IsValid())
		{
			Sound->SoundClassObject = TemplateSoundWave->SoundClassObject;
			Sound->ConcurrencySet = TemplateSoundWave->ConcurrencySet;
			Sound->CompressionQuality = TemplateSoundWave->CompressionQuality;
			Sound->SoundAssetCompressionType = TemplateSoundWave->SoundAssetCompressionType;

			Sound->ResetInvalidTemplateArguments();
		}

		// Starting or ending on a non-zero sample creates a pop unless you have very specific use cases
		// (doesn't apply to ambi)
		// Skip the check for DC offsets when audio data is empty
		if (NumFrames > 0 && !bIsFuMa && !bIsAmbiX)
		{
			int32 EndChannelNonZero = 0;
			int32 StartChannelNonZero = 0;
			constexpr int16 LogDCLimit = 100; // amount of non zero dc we allow before we nag
			for (int32 Chan = 0; Chan < ChannelCount; ++Chan)
			{
				const int16* FirstFrame = (const int16*)WaveInfo.SampleDataStart;
				const int16* LastFrame = FirstFrame + (NumFrames-1)*ChannelCount;

				FirstFrame += Chan;
				LastFrame += Chan;

				if (FMath::Abs(FirstFrame[0]) > LogDCLimit)
				{
					StartChannelNonZero++;
				}

				if (FMath::Abs(LastFrame[0]) > LogDCLimit)
				{
					EndChannelNonZero++;
				}
			}

			if (EndChannelNonZero && StartChannelNonZero)
			{
				Warn->Logf(ELogVerbosity::Log, TEXT("Imported audio has DC offsets larger than %d at the end of %d channel%s and the start of %d channel%s - may pop and not loop well: '%s'"), LogDCLimit, EndChannelNonZero, EndChannelNonZero > 1 ? TEXT("s") : TEXT(""), StartChannelNonZero, StartChannelNonZero > 1 ? TEXT("s") : TEXT(""), *Name.ToString());
			}
			else if (EndChannelNonZero)
			{
				Warn->Logf(ELogVerbosity::Log, TEXT("Imported audio has DC offsets larger than %d at the end of %d channel%s - may pop and not loop well: '%s'"), LogDCLimit, EndChannelNonZero, EndChannelNonZero > 1 ? TEXT("s") : TEXT(""), *Name.ToString());
			}
			else if (StartChannelNonZero)
			{
				Warn->Logf(ELogVerbosity::Log, TEXT("Imported audio has DC offsets larger than %d at the start of %d channel%s - may pop and not loop well: '%s'"), LogDCLimit, StartChannelNonZero, StartChannelNonZero > 1 ? TEXT("s") : TEXT(""), *Name.ToString());
			}
		}

		if (ChannelCount > 2)
		{
			// We need to deinterleave the raw PCM data in the multi-channel file reuse a scratch buffer
			TArray<int16> DeinterleavedAudioScratchBuffer;

			// Store the array of raw .wav files we're going to create from the deinterleaved int16 data
			TArray<uint8> RawChannelWaveData[SPEAKER_Count];

			// Ptr to the pcm data of the imported sound wave
			int16* SampleDataBuffer = (int16*)WaveInfo.SampleDataStart;

			int32 TotalSize = 0;

			Sound->ChannelOffsets.Empty(SPEAKER_Count);
			Sound->ChannelOffsets.AddZeroed(SPEAKER_Count);

			Sound->ChannelSizes.Empty(SPEAKER_Count);
			Sound->ChannelSizes.AddZeroed(SPEAKER_Count);

			TArray<int32> ChannelIndices;
			if (ChannelCount == 4)
			{
				ChannelIndices = {
					SPEAKER_FrontLeft,
					SPEAKER_FrontRight,
					SPEAKER_LeftSurround,
					SPEAKER_RightSurround
				};
			}
			else if (ChannelCount == 6)
			{
				ChannelIndices = {
					SPEAKER_FrontLeft,
					SPEAKER_FrontRight,
					SPEAKER_FrontCenter,
					SPEAKER_LowFrequency,
					SPEAKER_LeftSurround,
					SPEAKER_RightSurround
				};
			}
			else if (ChannelCount == 8)
			{
				ChannelIndices = {
					SPEAKER_FrontLeft,
					SPEAKER_FrontRight,
					SPEAKER_FrontCenter,
					SPEAKER_LowFrequency,
					SPEAKER_LeftSurround,
					SPEAKER_RightSurround,
					SPEAKER_LeftBack,
					SPEAKER_RightBack
				};
			}
			else
			{
				Warn->Logf(ELogVerbosity::Error, TEXT("Wave file '%s' has unsupported number of channels %d"), *Name.ToString(), ChannelCount);
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
				return nullptr;
			}

			// Make some new sound waves
			check(ChannelCount == ChannelIndices.Num());
			for (int32 Chan = 0; Chan < ChannelCount; ++Chan)
			{
				// Build the deinterleaved buffer for the channel
				DeinterleavedAudioScratchBuffer.Empty(NumFrames);
				for (int32 Frame = 0; Frame < NumFrames; ++Frame)
				{
					const int32 SampleIndex = Frame * ChannelCount + Chan;
					DeinterleavedAudioScratchBuffer.Add(SampleDataBuffer[SampleIndex]);
				}

				// Now create a sound wave asset
				SerializeWaveFile(RawChannelWaveData[Chan], (uint8*)DeinterleavedAudioScratchBuffer.GetData(), NumFrames * sizeof(int16), 1, *WaveInfo.pSamplesPerSec);

				// The current TotalSize is the "offset" into the bulk data for this sound wave
				Sound->ChannelOffsets[ChannelIndices[Chan]] = TotalSize;

				// "ChannelSize" is the size of the .wav file representing this channel of data
				const int32 ChannelSize = RawChannelWaveData[Chan].Num();

				// Store it in the sound wave
				Sound->ChannelSizes[ChannelIndices[Chan]] = ChannelSize;

				// TotalSize is the sum of all ChannelSizes
				TotalSize += ChannelSize;
			}

			// Now we have an array of mono .wav files in the format that the SoundSurroundFactory expects
			// copy the data into the bulk byte data

			// Get the raw data bulk byte pointer and copy over the .wav files we generated

			FUniqueBuffer EditableBuffer = FUniqueBuffer::Alloc(TotalSize);
			uint8* LockedData = (uint8*)EditableBuffer.GetData();
			int16* LockedDataInt16 = reinterpret_cast<int16*>(LockedData);

			int32 RawDataOffset = 0;


			if (bIsAmbiX || bIsFuMa)
			{
				check(ChannelCount == 4);
				// Flag that this is an ambisonics file
				Sound->bIsAmbisonics = true;
			}
			if (bIsFuMa)
			{
				int32 FuMaChannelIndices[4] = { 0, 2, 3, 1 };
				const float ScalerPlus3dB = Audio::ConvertToLinear(3.0f);

				for (int32 ChannelIndex : FuMaChannelIndices)
				{
					const int32 ChannelSize = RawChannelWaveData[ChannelIndex].Num();
					FMemory::Memcpy(LockedData + RawDataOffset, RawChannelWaveData[ChannelIndex].GetData(), ChannelSize);
					RawDataOffset += ChannelSize;

//  TODO: make sure this isn't already being done somewhere else, conversion sounds wrong when gain is applied
					// scale zeroth channel
//					if (ChannelIndex == 0)
//					{
// 						for (int32 i = 0; i < ChannelSize; ++i)
// 						{
// 							LockedData[i] = static_cast<int16>(static_cast<float>(LockedData[i]) * ScalerPlus3dB);
// 						}
//					}
				}
			}
			else
			{
				for (int32 Chan = 0; Chan < ChannelCount; ++Chan)
				{
					const int32 ChannelSize = RawChannelWaveData[Chan].Num();
					FMemory::Memcpy(LockedData + RawDataOffset, RawChannelWaveData[Chan].GetData(), ChannelSize);
					RawDataOffset += ChannelSize;
				}
			}

			Sound->RawData.UpdatePayload(EditableBuffer.MoveToShared());
		}
		else
		{
			// If this sound existed previously, we need to clear out any stale multichannel data on the sound wave in the case this is a reimport from multichannel to mono/stereo
			if (ExistingSound)
			{
				ExistingSound->ChannelOffsets.Reset();
				ExistingSound->ChannelSizes.Reset();
				ExistingSound->bIsAmbisonics = false;
			}
			

			// For mono and stereo assets, just copy the data into the buffer
			// Clone directly as a param so that if anyone MoveToUniques it then its a steal not a copy.
			Sound->RawData.UpdatePayload(FSharedBuffer::Clone(Buffer, RawWaveDataBufferSize));

		}

		Sound->Duration = SoundDuration;
		Sound->SetImportedSampleRate(*WaveInfo.pSamplesPerSec);
		Sound->SetSampleRate(*WaveInfo.pSamplesPerSec);
		Sound->NumChannels = ChannelCount;
		Sound->TotalSamples = *WaveInfo.pSamplesPerSec * Sound->Duration;

		// Store the current file path and timestamp for re-import purposes
		Sound->AssetImportData->Update(CurrentFilename);

		int TotalNum = WaveInfo.WaveCues.Num() + WaveInfo.WaveSampleLoops.Num();
		Sound->CuePoints.Reset(TotalNum);
		TSet<int32> WaveCuePointIDs;
		WaveCuePointIDs.Reserve(WaveInfo.WaveCues.Num());

		// Start with WaveCues
		for (FWaveCue& WaveCue : WaveInfo.WaveCues)
		{
			FSoundWaveCuePoint NewCuePoint;
			NewCuePoint.CuePointID = static_cast<int32>(WaveCue.CuePointID);
			NewCuePoint.FrameLength = static_cast<int64>(WaveCue.SampleLength);
			NewCuePoint.FramePosition = static_cast<int64>(WaveCue.Position);
			NewCuePoint.Label = WaveCue.Label;
			Sound->CuePoints.Add(NewCuePoint);
			WaveCuePointIDs.Add(NewCuePoint.CuePointID);
		}

		// add Sample Loops to end or replace existing WaveCue 
		// with matching SampleLoop to avoid duplicated/reduntant loops
		bool FoundInvalidSampleLoops = false;
		for (FWaveSampleLoop& SampleLoop : WaveInfo.WaveSampleLoops)
		{
			FSoundWaveCuePoint NewCuePoint;
			NewCuePoint.bIsLoopRegion = true;
			NewCuePoint.CuePointID = static_cast<int32>(SampleLoop.LoopID);
			NewCuePoint.FramePosition = static_cast<int64>(SampleLoop.StartFrame);
			NewCuePoint.FrameLength = static_cast<int64>(SampleLoop.EndFrame) - static_cast<int64>(SampleLoop.StartFrame);
			if (SampleLoop.EndFrame <= SampleLoop.StartFrame)
			{
				Warn->Logf(ELogVerbosity::Error, 
					TEXT("EndFrame must be greater than StartFrame when creating Cue Point from Sample Loop Region! LoopID = %d, StartFrame = %d, EndFrame = %d"), 
					SampleLoop.LoopID, SampleLoop.StartFrame, SampleLoop.EndFrame);

				FoundInvalidSampleLoops = true;
				NewCuePoint.FrameLength = 0;
			}

			if (WaveCuePointIDs.Contains(NewCuePoint.CuePointID))
			{
				for (FSoundWaveCuePoint& CuePoint : Sound->CuePoints)
				{
					if (CuePoint.CuePointID == NewCuePoint.CuePointID)
					{
						//Initial CuePoint only has values for a marker
						//and not a loop region, so loop values set here
						CuePoint.bIsLoopRegion = true;
						CuePoint.FrameLength = NewCuePoint.FrameLength;
						break;
					}
				}
			}
			else
			{
				Sound->CuePoints.Add(NewCuePoint);
			}
		}

#if WITH_EDITORONLY_DATA
		// Setup the cue points
		Sound->CuePointOrigin = Sound->CuePoints.IsEmpty() ? ESoundWaveCuePointOrigin::MarkerTransformation : ESoundWaveCuePointOrigin::WaveFile;
#endif //WITH_EDITORONLY_DATA

		// fail import if we found invalid sample loops
		// each invalid sample loop is logged
		if (FoundInvalidSampleLoops)
		{
			FText InvalidSamplesText = NSLOCTEXT("SoundFactory", "Invalid Sample Loops", "Found sample loops with invalid start and end frames. See logs for more info.");
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("SoundFactory", "Import Failed", "Import failed for {0}: {1}"), FText::FromString(Name.ToString()), InvalidSamplesText));
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		// If we've read some time-code.
		if (WaveInfo.TimecodeInfo)
		{
			Sound->SetTimecodeInfo(*WaveInfo.TimecodeInfo);
		}
		else
		{
			Sound->SetTimecodeInfo(FSoundWaveTimecodeInfo{});
		}
				
		// Compressed data is now out of date.
		Sound->InvalidateCompressedData(true /* bFreeResources */);

		// If stream caching is enabled, we need to make sure this asset is ready for playback.
		if (Sound->IsStreaming(nullptr))
		{
			Sound->LoadZerothChunk();
		}

		Sound->PostImport();

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Sound);

		if (ExistingSound && bUseExistingSettings)
		{
			// Call PostEditChange() to update text to speech
			Sound->PostEditChange();
		}

		// if we're auto creating a default cue
		if (bAutoCreateCue)
		{
			CreateSoundCue(Sound, bMoveCue ? CuePackage : InParent, Flags, bIncludeAttenuationNode, bIncludeModulatorNode, bIncludeLoopingNode, CueVolume);
		}

		for (UAudioComponent* AudioComponent : ComponentsToRestart)
		{
			AudioComponent->Play();
		}

		Sound->SetRedrawThumbnail(true);

		Audio::Analytics::RecordEvent_Usage(TEXT("SoundFactory.SoundWaveImported"));

		return Sound;
	}
	else
	{
		// Unrecognized sound format
		Warn->Logf(ELogVerbosity::Error, TEXT("Unrecognized sound format '%s' in %s"), FileType, *Name.ToString());
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
	}

	return nullptr;
}

bool ShowUseTemplateDialog(const FString& SoundWaveName, bool& bOutRememberChoice)
{
	bool bUserChoice = false;
	bOutRememberChoice = false;

	TSharedPtr<SWindow> DialogWindow = SNew(SWindow)
		.Title(FText::FromString("SoundWave Import Options"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false).SupportsMaximize(false);

	TWeakPtr<SWindow> WeakWindow = DialogWindow;

	TSharedPtr<SCheckBox> RememberCheckBox;

	DialogWindow->SetContent(
		SNew(SBorder)
		.Padding(10)
		[
			SNew(SVerticalBox)

				+ SVerticalBox::Slot().AutoHeight().Padding(5)
				[
					SNew(STextBlock)
						.AutoWrapText(true)
						.Text(FText::Format(
							NSLOCTEXT("SoundFactory", "UseSoundWaveTemplate", "Use the selected Sound Wave '{0}' as a template for sound(s) being imported?"),
							FText::FromString(SoundWaveName)))
				]

			+ SVerticalBox::Slot().AutoHeight().Padding(5)
				[
					SAssignNew(RememberCheckBox, SCheckBox)
						[
							SNew(STextBlock).Text(NSLOCTEXT("SoundFactory", "RememberChoice", "Remember my choice"))
						]
				]

				+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).Padding(5)
				[
					SNew(SUniformGridPanel).SlotPadding(5)

						+ SUniformGridPanel::Slot(0, 0)
						[
							SNew(SButton)
								.Text(NSLOCTEXT("Generic", "Yes", "Yes"))
								.OnClicked_Lambda([WeakWindow, &bUserChoice]() 
									{
										bUserChoice = true;

										if (TSharedPtr<SWindow> PinnedWindow = WeakWindow.Pin())
										{
											FSlateApplication::Get().RequestDestroyWindow(PinnedWindow.ToSharedRef());
										}
										
										return FReply::Handled();
									})
						]

					+ SUniformGridPanel::Slot(1, 0)
						[
							SNew(SButton)
								.Text(NSLOCTEXT("Generic", "No", "No"))
								.OnClicked_Lambda([WeakWindow]()
									{
										if (TSharedPtr<SWindow> PinnedWindow = WeakWindow.Pin())
										{
											FSlateApplication::Get().RequestDestroyWindow(PinnedWindow.ToSharedRef());
										}
										
										return FReply::Handled();
									})
						]
				]
		]);

	FSlateApplication::Get().AddModalWindow(DialogWindow.ToSharedRef(), nullptr);

	bOutRememberChoice = RememberCheckBox.IsValid() && RememberCheckBox->IsChecked();

	return bUserChoice;
}

void USoundFactory::SuppressImportDialogs()
{
	SuppressImportDialogOptions = ESuppressImportDialog::Overwrite | ESuppressImportDialog::UseTemplate;
}

void USoundFactory::UpdateTemplate()
{
	EUseTemplateSoundWaveDuringAssetImport& UseTemplateSoundWaveEditorSetting = GetMutableDefault<UAudioEditorPerProjectUserSettings>()->UseTemplateSoundWaveDuringAssetImport;
	const bool bUseTemplate = UseTemplateSoundWaveEditorSetting != EUseTemplateSoundWaveDuringAssetImport::NeverUse;

	if (!IsAutomatedImport() && bUseTemplate && !TemplateSoundWave.IsValid() && !(SuppressImportDialogOptions & ESuppressImportDialog::UseTemplate))
	{
		SuppressImportDialogOptions |= ESuppressImportDialog::UseTemplate;

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FAssetData> SelectedAssets;
		ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

		// If SelectedAssets.Num() != 1, no TemplateSoundWave is used
		if (SelectedAssets.Num() == 1)
		{
			if (USoundWave* SoundWave = Cast<USoundWave>(SelectedAssets[0].GetAsset()))
			{
				if (UseTemplateSoundWaveEditorSetting == EUseTemplateSoundWaveDuringAssetImport::AlwaysUse)
				{
					TemplateSoundWave = SoundWave;

					return;
				}

				check(UseTemplateSoundWaveEditorSetting == EUseTemplateSoundWaveDuringAssetImport::AlwaysPrompt)

				bool bRememberSelectedChoice = false; 
				const bool bUseTemplateSoundWave = ShowUseTemplateDialog(SoundWave->GetName(), bRememberSelectedChoice);

				if (bUseTemplateSoundWave)
				{
					TemplateSoundWave = SoundWave;
				}

				if (bRememberSelectedChoice)
				{
					if (bUseTemplateSoundWave)
					{
						UseTemplateSoundWaveEditorSetting = EUseTemplateSoundWaveDuringAssetImport::AlwaysUse;
					}
					else
					{
						UseTemplateSoundWaveEditorSetting = EUseTemplateSoundWaveDuringAssetImport::NeverUse;
					}
				}
				else
				{
					UseTemplateSoundWaveEditorSetting = EUseTemplateSoundWaveDuringAssetImport::AlwaysPrompt;
				}
			}
		}
	}
}

void USoundFactory::CleanUp()
{
	SuppressImportDialogOptions = ESuppressImportDialog::None;
	TemplateSoundWave.Reset();
}
