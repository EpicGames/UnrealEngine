// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundMixFactory.h"

#include "Sound/SoundMix.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundMixFactory)

class FFeedbackContext;
class UClass;
class UObject;

USoundMixFactory::USoundMixFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = USoundMix::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundMixFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundMix* Mix = NewObject<USoundMix>(InParent, InName, Flags);

	return Mix;
}
