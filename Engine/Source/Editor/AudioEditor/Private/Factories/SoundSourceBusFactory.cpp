// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundSourceBusFactory.h"

#include "Sound/SoundSourceBus.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSourceBusFactory)

class FFeedbackContext;
class UClass;
class UObject;

USoundSourceBusFactory::USoundSourceBusFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundSourceBus::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundSourceBusFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundSourceBus* SoundSourceBus = NewObject<USoundSourceBus>(InParent, InName, Flags);

	return SoundSourceBus;
}

bool USoundSourceBusFactory::CanCreateNew() const
{
	return true;
}
