// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundSourceEffectFactory.h"

#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Factories/SoundFactoryUtility.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"
#include "Sound/SoundEffectSource.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSourceEffectFactory)

class FFeedbackContext;
class UClass;
class UObject;

#define LOCTEXT_NAMESPACE "AudioEditorFactories"

USoundSourceEffectFactory::USoundSourceEffectFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundEffectSourcePreset::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	SoundEffectSourcepresetClass = nullptr;
}

bool USoundSourceEffectFactory::ConfigureProperties()
{
	SoundEffectSourcepresetClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(USoundEffectSourcePreset::StaticClass());

	const FText TitleText = LOCTEXT("CreateSoundSourceEffectOptions", "Pick Source Effect Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, USoundEffectSourcePreset::StaticClass());

	if (bPressedOk)
	{
		SoundEffectSourcepresetClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* USoundSourceEffectFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundEffectSourcePreset* NewSoundEffectSourcePreset = nullptr;
	if (SoundEffectSourcepresetClass != nullptr)
	{
		NewSoundEffectSourcePreset = NewObject<USoundEffectSourcePreset>(InParent, SoundEffectSourcepresetClass, InName, Flags);
	}
	return NewSoundEffectSourcePreset;
}

bool USoundSourceEffectFactory::CanCreateNew() const
{
	return true;
}

USoundSourceEffectChainFactory::USoundSourceEffectChainFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundEffectSourcePresetChain::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundSourceEffectChainFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<USoundEffectSourcePresetChain>(InParent, InName, Flags);
}

bool USoundSourceEffectChainFactory::CanCreateNew() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
