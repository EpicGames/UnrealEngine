// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundSubmixEffectFactory.h"

#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Factories/SoundFactoryUtility.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"
#include "Sound/SoundEffectSubmix.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmixEffectFactory)

class FFeedbackContext;
class UClass;
class UObject;

#define LOCTEXT_NAMESPACE "AudioEditorFactories"

USoundSubmixEffectFactory::USoundSubmixEffectFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundEffectSubmixPreset::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	SoundEffectSubmixPresetClass = nullptr;
}

bool USoundSubmixEffectFactory::ConfigureProperties()
{
	SoundEffectSubmixPresetClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(USoundEffectSubmixPreset::StaticClass());

	const FText TitleText = LOCTEXT("CreateSoundSubmixEffectOptions", "Pick Submix Effect Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, USoundEffectSubmixPreset::StaticClass());

	if (bPressedOk)
	{
		SoundEffectSubmixPresetClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* USoundSubmixEffectFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundEffectSubmixPreset* NewSoundEffectSubmixPreset = nullptr;
	if (SoundEffectSubmixPresetClass != nullptr)
	{
		NewSoundEffectSubmixPreset = NewObject<USoundEffectSubmixPreset>(InParent, SoundEffectSubmixPresetClass, InName, Flags);
	}
	return NewSoundEffectSubmixPreset;
}

bool USoundSubmixEffectFactory::CanCreateNew() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE
