// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SoundSubmixFactory.h"

#include "AudioDeviceManager.h"
#include "Engine/Engine.h"
#include "Sound/SoundSubmix.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmixFactory)

class FFeedbackContext;
class UClass;
class UObject;

USoundSubmixFactory::USoundSubmixFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundSubmix::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundSubmixFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundSubmix* SoundSubmix = NewObject<USoundSubmix>(InParent, InName, Flags);

	class FAudioDeviceManager* AudioDeviceManager = GEngine ? GEngine->GetAudioDeviceManager() : nullptr;
	if (AudioDeviceManager)
	{
		AudioDeviceManager->InitSoundSubmixes();
	}

	return SoundSubmix;
}

bool USoundSubmixFactory::CanCreateNew() const
{
	return true;
}

// Soundfield Submix Factory: 

USoundfieldSubmixFactory::USoundfieldSubmixFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundfieldSubmix::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundfieldSubmixFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundfieldSubmix* SoundSubmix = NewObject<USoundfieldSubmix>(InParent, Name, Flags);

	class FAudioDeviceManager* AudioDeviceManager = GEngine ? GEngine->GetAudioDeviceManager() : nullptr;
	if (AudioDeviceManager)
	{
		AudioDeviceManager->InitSoundSubmixes();
	}

	return SoundSubmix;
}

bool USoundfieldSubmixFactory::CanCreateNew() const
{
	return true;
}

// Endpoint Submix Factory:

UEndpointSubmixFactory::UEndpointSubmixFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UEndpointSubmix::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UEndpointSubmixFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UEndpointSubmix* SoundSubmix = NewObject<UEndpointSubmix>(InParent, Name, Flags);

	class FAudioDeviceManager* AudioDeviceManager = GEngine ? GEngine->GetAudioDeviceManager() : nullptr;
	if (AudioDeviceManager)
	{
		AudioDeviceManager->InitSoundSubmixes();
	}

	return SoundSubmix;
}

bool UEndpointSubmixFactory::CanCreateNew() const
{
	return true;
}


// Soundfield Endpoint Submix Factory:

USoundfieldEndpointSubmixFactory::USoundfieldEndpointSubmixFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundfieldEndpointSubmix::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundfieldEndpointSubmixFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundfieldEndpointSubmix* SoundSubmix = NewObject<USoundfieldEndpointSubmix>(InParent, Name, Flags);

	class FAudioDeviceManager* AudioDeviceManager = GEngine ? GEngine->GetAudioDeviceManager() : nullptr;
	if (AudioDeviceManager)
	{
		AudioDeviceManager->InitSoundSubmixes();
	}

	return SoundSubmix;
}

bool USoundfieldEndpointSubmixFactory::CanCreateNew() const
{
	return true;
}
