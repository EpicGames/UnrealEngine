// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFindInSoundClassGraph.h"

#include "BlueprintUtilities.h"
#include "Framework/Application/SlateApplication.h"
#include "GraphEditAction.h"
#include "GraphEditor.h"
#include "Layout/WidgetPath.h"
#include "Sound/SoundClass.h"
#include "SoundClassGraph/SoundClassGraphNode.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SoundClassEditor"

void SFindInSoundClassGraph::Construct(const FArguments& InArgs, TSharedPtr<SGraphEditor> InGraphEditor)
{
	check(InGraphEditor.IsValid());
	check(InGraphEditor->GetCurrentGraph() != nullptr);
	
	GraphEditorPtr = InGraphEditor;
	OnResultActivated = InArgs._OnSearchResultActivated;
	
	// Update search results when the graph gets modified
	InGraphEditor->GetCurrentGraph()->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSPLambda(this, [this](const FEdGraphEditAction& Action)
	{
		if (!SearchValue.IsEmpty())
		{
			// Last case is to detect "Undo/Redo" of node addition/creation, as these don't appear as AddNode or RemoveNode.
			if (Action.Action == EEdGraphActionType::GRAPHACTION_AddNode || Action.Action == EEdGraphActionType::GRAPHACTION_RemoveNode 
				|| (Action.Action == EEdGraphActionType::GRAPHACTION_Default && Action.Nodes.IsEmpty() && Action.Graph == nullptr))
			{
				InitiateSearch();
			}
		}
	}));

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SAssignNew(SearchTextField, SSearchBox)
				.HintText(LOCTEXT("Find_GraphSearchHint", "Search"))
				.OnTextChanged(this, &SFindInSoundClassGraph::OnSearchTextChanged)
				.OnTextCommitted(this, &SFindInSoundClassGraph::OnSearchTextCommitted)
				.DelayChangeNotificationsWhileTyping(true)
				.DelayChangeNotificationsWhileTypingSeconds(0.25f)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f)
		[
			SNew(STextBlock)
			.Visibility_Lambda([this](){ return SearchTextField->GetText().IsEmptyOrWhitespace() ? EVisibility::Hidden : EVisibility::Visible; })
			.Text_Lambda([this]() { return FText::Format(LOCTEXT("Find_NumResultsFmt", "{0} matching {0}|plural(one=node,other=nodes)"), FoundNodeCount); })
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(4.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(ListView, SListViewType)
				.ListItemsSource(&ItemsFound)
				.OnGenerateRow(this, &SFindInSoundClassGraph::OnGenerateRow)
				.OnMouseButtonDoubleClick(this, &SFindInSoundClassGraph::OnSelectionDoubleClick)
				.SelectionMode(ESelectionMode::Single)
				.OnKeyDownHandler(FOnKeyDown::CreateSPLambda(this, [this](const FGeometry&, const FKeyEvent& Event)
				{
					if (Event.GetKey() == EKeys::Enter || Event.GetKey() == EKeys::SpaceBar)
					{
						TArray<FSearchResult> SelectedItems = ListView->GetSelectedItems();
						if (!SelectedItems.IsEmpty())
						{
							OnSelectionDoubleClick(SelectedItems[0]);
							return FReply::Handled();
						}
					}
						
					return FReply::Unhandled();
				}))
			]
		]
	];
}

void SFindInSoundClassGraph::FocusForUse()
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
}

void SFindInSoundClassGraph::FocusForUse(const FString& NewSearchTerms)
{
	FocusForUse();

	if (!NewSearchTerms.IsEmpty())
	{
		SearchTextField->SetText(FText::FromString(NewSearchTerms));
		InitiateSearch();
	}
}

void SFindInSoundClassGraph::OnSearchTextChanged(const FText& Text)
{
	SearchValue = Text.ToString();
	InitiateSearch();
}

void SFindInSoundClassGraph::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		SearchValue = Text.ToString();
		InitiateSearch();
	}
}

void SFindInSoundClassGraph::InitiateSearch()
{
	TArray<FString> Tokens;
	if (SearchValue.Contains("\"") && SearchValue.ParseIntoArray(Tokens, TEXT("\""), true) > 0)
	{
		for (FString& TokenIt : Tokens)
		{
			// we have the token, we don't need the quotes anymore, they'll just confuse the comparison later on
			TokenIt = TokenIt.TrimQuotes();
			// We remove the spaces as all later comparison strings will also be de-spaced
			TokenIt = TokenIt.Replace(TEXT(" "), TEXT(""));
		}

		// due to being able to handle multiple quoted blocks like ("Make Epic" "Game Now") we can end up with
		// and empty string between (" ") blocks so this simply removes them
		struct FRemoveMatchingStrings
		{
			bool operator()(const FString& RemovalCandidate) const
			{
				return RemovalCandidate.IsEmpty();
			}
		};
		Tokens.RemoveAll(FRemoveMatchingStrings());
	}
	else
	{
		// unquoted search equivalent to a match-any-of search
		SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);
	}

	FoundNodeCount = 0;
	ItemsFound.Empty();
	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchValue);
		MatchTokens(Tokens);
	}

	struct FSearchResultLexicalLess
	{
		[[nodiscard]] bool operator()(const FSearchResult& Lhs, const FSearchResult& Rhs) const
		{
			// Push invalid results at the bottom, if any
			if (!Lhs.IsValid())
			{
				return false;
			}
			if (!Rhs.IsValid())
			{
				return true;
			}
			
			// Otherwise sort lexically ascending
			return Lhs->SoundClass.GetName() < Rhs->SoundClass.GetName();
		}
	};
	ItemsFound.StableSort(FSearchResultLexicalLess());
	ListView->RequestListRefresh();
}

void SFindInSoundClassGraph::MatchTokens(const TArray<FString>& Tokens)
{
	if (TSharedPtr<SGraphEditor> GraphEditor = GraphEditorPtr.Pin())
	{
		MatchTokensInGraph(GraphEditor->GetCurrentGraph(), Tokens);
	}
}

void SFindInSoundClassGraph::MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens)
{
	if (!Graph)
	{
		return;
	}
	
	for (auto It(Graph->Nodes.CreateConstIterator()); It; ++It)
	{
		UEdGraphNode* Node = *It;

		FString NodeSearchString = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString().Replace(TEXT(" "), TEXT(""));

		if (StringMatchesSearchTokens(Tokens, NodeSearchString))
		{
			ItemsFound.Emplace(CastChecked<USoundClassGraphNode>(Node));
			FoundNodeCount++;
		}
	}

	for (const UEdGraph* Subgraph : Graph->SubGraphs)
	{
		MatchTokensInGraph(Subgraph, Tokens);
	}
}

TSharedRef<ITableRow> SFindInSoundClassGraph::OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerList)
{
	return SNew(STableRow<FSearchResult>, OwnerList)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2.f)
		[
			SNew(STextBlock)
				.Text(FText::FromString(GetNameSafe(InItem->SoundClass)))
				.HighlightText_Lambda([this](){ return HighlightText; })
		]
	];
}

void SFindInSoundClassGraph::OnSelectionDoubleClick(FSearchResult Item)
{
	if (USoundClassGraphNode* Node = Item.Get())
	{
		OnResultActivated.ExecuteIfBound(Node);
	}
}

bool SFindInSoundClassGraph::StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString)
{
	bool bFoundAllTokens = true;
	//search the entry for each token, it must have all of them to pass
	for (const FString& Token : Tokens)
	{
		if (!ComparisonString.Contains(Token))
		{
			bFoundAllTokens = false;
			break;
		}
	}
	return bFoundAllTokens;
}

#undef LOCTEXT_NAMESPACE
