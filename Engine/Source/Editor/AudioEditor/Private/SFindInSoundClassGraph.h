// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SGraphEditor;
class SSearchBox;
class UEdGraph;
class USoundClassGraphNode;

class SFindInSoundClassGraph final : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnResultActivated, USoundClassGraphNode*)
	
	SLATE_BEGIN_ARGS(SFindInSoundClassGraph) {}
		SLATE_EVENT(FOnResultActivated, OnSearchResultActivated)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SGraphEditor> InGraphEditor);

	/** Focuses this widget's search box */
	void FocusForUse();
	/** Focuses this widget's search box, and Initiates a search with given SearchTerms */
	void FocusForUse(const FString& NewSearchTerms);

private:
	typedef TWeakObjectPtr<USoundClassGraphNode> FSearchResult;
	typedef SListView<FSearchResult> SListViewType;

	/** Called when user changes the text they are searching for */
	void OnSearchTextChanged(const FText& Text);

	/** Called when user changes commits text to the search box */
	void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/** Called when user double-clicks on a new result */
	void OnSelectionDoubleClick(FSearchResult Item);

	/** Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerList);

	/** Begins the search based on the SearchValue */
	void InitiateSearch();

	/** Find any results that contain all of the tokens */
	void MatchTokens(const TArray<FString>& Tokens);

	/** Find any results that contain all of the tokens in provided graph */
	void MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens);

	/** Determines if a string matches the search tokens */
	static bool StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString);

	/** Pointer back to the Graph Editor of the Sound Class editor that owns us */
	TWeakPtr<SGraphEditor> GraphEditorPtr;

	/** The list view displays the results */
	TSharedPtr<SListViewType> ListView;

	/** The search text box */
	TSharedPtr<SSearchBox> SearchTextField;
	
	/** The callback to execute when a search result is activated by the user (provides the node corresponding to the result) */
	FOnResultActivated OnResultActivated;

	/** This buffer stores the currently displayed results */
	TArray<FSearchResult> ItemsFound;

	/** The string to highlight in the results */
	FText HighlightText;

	/** The string to search for */
	FString SearchValue;

	/** Counters for number of results */
	uint32 FoundNodeCount = 0;
};
