// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNodeSoundBase.h"

#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

void SGraphNodeSoundBase::Construct(const FArguments& InArgs, class USoundCueGraphNode* InNode)
{
	this->GraphNode = InNode;
	this->SoundNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeSoundBase::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
		NSLOCTEXT("SoundNode", "SoundNodeAddPinButton", "Add input"),
		NSLOCTEXT("SoundNode", "SoundNodeAddPinButton_Tooltip", "Adds an input to the sound node")
	);

	FMargin AddPinPadding = Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	OutputBox->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Center)
	.Padding(AddPinPadding)
	[
		AddPinButton
	];
}

EVisibility SGraphNodeSoundBase::IsAddPinButtonVisible() const
{
	EVisibility ButtonVisibility = SGraphNode::IsAddPinButtonVisible();
	if (ButtonVisibility == EVisibility::Visible)
	{
		if (!SoundNode->CanAddInputPin())
		{
			ButtonVisibility = EVisibility::Collapsed;
		}
	}
	return ButtonVisibility;
}

FReply SGraphNodeSoundBase::OnAddPin()
{
	SoundNode->AddInputPin();

	return FReply::Handled();
}
