// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "SGraphNode.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SVerticalBox;
class USoundCueGraphNode;

class SGraphNodeSoundBase : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeSoundBase){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class USoundCueGraphNode* InNode);

protected:
	// SGraphNode Interface
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual EVisibility IsAddPinButtonVisible() const override;
	virtual FReply OnAddPin() override;

private:
	USoundCueGraphNode* SoundNode;
};
