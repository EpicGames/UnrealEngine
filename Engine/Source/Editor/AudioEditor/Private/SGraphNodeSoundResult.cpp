// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNodeSoundResult.h"

#include "GenericPlatform/ICursor.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "SoundCueGraph/SoundCueGraphNode_Base.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

void SGraphNodeSoundResult::Construct(const FArguments& InArgs, USoundCueGraphNode_Base* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

TSharedRef<SWidget> SGraphNodeSoundResult::CreateNodeContentArea()
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(0, 3))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				// LEFT
				SAssignNew(LeftNodeBox, SVerticalBox)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Graph.SoundResultNode.Body"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				// RIGHT
				SAssignNew(RightNodeBox, SVerticalBox)
			]
		];
}
