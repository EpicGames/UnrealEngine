// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;

class SGraphNodeSoundResult : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeSoundResult){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class USoundCueGraphNode_Base* InNode);

protected:
	// SGraphNode interface
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	// End of SGraphNode interface
};
