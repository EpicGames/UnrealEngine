// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSoundClassActionMenu.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "SGraphActionMenu.h"
#include "SlotBase.h"
#include "SoundClassGraph/SoundClassGraphSchema.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SSoundClassActionMenu"

void SSoundClassActionMenuItem::Construct(const FArguments& InArgs, TSharedPtr<FEdGraphSchemaAction> InAction, TWeakPtr<SSoundClassActionMenu> InOwner)
{
	check(InAction.IsValid());

	this->Owner = InOwner;

	bool bIsNewSoundClass = false;
	if (InAction->GetTypeId() == FSoundClassGraphSchemaAction_NewNode::StaticGetTypeId())
	{
		bIsNewSoundClass = true;
	}

	// The new sound class widget requires 2 lines as it has a text entry box also.	
	if( !bIsNewSoundClass )
	{
		this->ChildSlot
			[
				SNew(SHorizontalBox)
				.ToolTipText(InAction->GetTooltipDescription())
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
					.Text(InAction->GetMenuDescription())
					.HighlightText(InArgs._HighlightText)
				]
			];
	}
	else
	{
		TSharedRef<SWidget> NewSoundClassWidget = CreateNewSoundClassWidget(InAction->GetMenuDescription(), InAction->GetTooltipDescription(), FCoreStyle::GetDefaultFontStyle("Regular", 9), InAction);
		// Promote requires 2 'slots'
		this->ChildSlot
			[
				NewSoundClassWidget
			];
	}
}

TSharedRef<SWidget> SSoundClassActionMenuItem::CreateNewSoundClassWidget( const FText& DisplayText, const FText& InToolTip, const FSlateFontInfo& NameFont, TSharedPtr<FEdGraphSchemaAction>& InAction )
{
	FString ClassName;
	FSoundClassGraphSchemaAction_NewNode* Action = static_cast<FSoundClassGraphSchemaAction_NewNode*>(InAction.Get());
	if( Action )
	{
		ClassName = Action->NewSoundClassName;
	}

	return SNew( SVerticalBox )				
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(DisplayText)
			.Font(NameFont)
			.ToolTipText(InToolTip)
		]
	+SVerticalBox::Slot()
		.AutoHeight()
		[			
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( FMargin(3,0) )
			.VAlign(VAlign_Center) 
			[
				SNew(SEditableTextBox)
				.Text(FText::FromString(ClassName))
				.ToolTipText(InToolTip)
				.OnTextCommitted( this, &SSoundClassActionMenuItem::OnNewSoundClassNameEntered, InAction )
				.OnTextChanged( this, &SSoundClassActionMenuItem::OnNewSoundClassNameChanged, InAction )
				.SelectAllTextWhenFocused( true )
				.RevertTextOnEscape( true )
			]
		];

}

void SSoundClassActionMenuItem::OnNewSoundClassNameChanged( const FText& NewText, TSharedPtr<FEdGraphSchemaAction> InAction )
{
	FSoundClassGraphSchemaAction_NewNode* Action = static_cast<FSoundClassGraphSchemaAction_NewNode*>(InAction.Get());
	Action->NewSoundClassName = NewText.ToString();
}

void SSoundClassActionMenuItem::OnNewSoundClassNameEntered( const FText& NewText, ETextCommit::Type CommitInfo, TSharedPtr<FEdGraphSchemaAction> InAction )
{
	// Do nothing if we aborted
	if (CommitInfo != ETextCommit::OnEnter)
	{		
		return;
	}

	FSoundClassGraphSchemaAction_NewNode* Action = static_cast<FSoundClassGraphSchemaAction_NewNode*>(InAction.Get());
	Action->NewSoundClassName = *NewText.ToString();

	TArray< TSharedPtr<FEdGraphSchemaAction> > ActionList;
	ActionList.Add( InAction );

	Owner.Pin()->OnActionSelected(ActionList, ESelectInfo::OnKeyPress);
}

///////////////////////////////////////////////

SSoundClassActionMenu::~SSoundClassActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
}

void SSoundClassActionMenu::Construct( const FArguments& InArgs )
{
	this->GraphObj = InArgs._GraphObj;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;

	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5)
		[
			SNew(SBox)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionSelected(this, &SSoundClassActionMenu::OnActionSelected)
				.OnCreateWidgetForAction( SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(this, &SSoundClassActionMenu::OnCreateWidgetForAction) )
				.OnCollectAllActions(this, &SSoundClassActionMenu::CollectAllActions)
				.AutoExpandActionMenu(bAutoExpandActionMenu)
				.ShowFilterTextBox(false)
			]
		]
	);
}

void SSoundClassActionMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	// Build up the context object
	FGraphContextMenuBuilder ContextMenuBuilder(GraphObj);
	if (DraggedFromPins.Num() > 0)
	{
		ContextMenuBuilder.FromPin = DraggedFromPins[0];
	}

	// Determine all possible actions
	GraphObj->GetSchema()->GetGraphContextActions(ContextMenuBuilder);

	// Copy the added options back to the main list
	//@TODO: Avoid this copy
	OutAllActions.Append(ContextMenuBuilder);
}

TSharedRef<SWidget> SSoundClassActionMenu::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SSoundClassActionMenuItem, InCreateData->Action, SharedThis(this))
		.HighlightText(InCreateData->HighlightText);
}


void SSoundClassActionMenu::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType )
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		if ( GraphObj != NULL )
		{
			for ( int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++ )
			{
				TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

				if ( CurrentAction.IsValid() )
				{
					FSlateApplication::Get().DismissAllMenus();

					CurrentAction->PerformAction(GraphObj, DraggedFromPins, NewNodePosition);
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
