// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSoundCuePalette.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "EdGraph/EdGraphSchema.h"
#include "Modules/ModuleManager.h"
#include "SoundCueGraph/SoundCueGraphSchema.h"
#include "UObject/UObjectGlobals.h"

void SSoundCuePalette::Construct(const FArguments& InArgs)
{
	// Register with the Class Viewer to rebuild the palette when the class viewer filter has changed
	FilterData.InitOptions = MakeShared<FClassViewerInitializationOptions>();
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	FilterData.ClassFilter = ClassViewerModule.CreateClassFilter(*FilterData.InitOptions);
	FilterData.FilterFuncs = ClassViewerModule.CreateFilterFuncs();
	ClassViewerModule.GetOnGlobalClassViewerFilterModified().AddRaw(this, &SSoundCuePalette::OnGlobalClassViewerFilterModified);	
	GetMutableDefault<USoundCueGraphSchema>()->UpdateSoundNodeList(FilterData);
	
	// Auto expand the palette as there's so few nodes
	SGraphPalette::Construct(SGraphPalette::FArguments().AutoExpandActionMenu(true));
}

SSoundCuePalette::~SSoundCuePalette()
{	
	if (FClassViewerModule* ClassViewerModule = FModuleManager::Get().GetModulePtr<FClassViewerModule>("ClassViewer"))
	{
		ClassViewerModule->GetOnGlobalClassViewerFilterModified().RemoveAll(this);
	}
}

void SSoundCuePalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FGraphActionMenuBuilder ActionMenuBuilder;

	// Determine all possible actions
	GetDefault<USoundCueGraphSchema>()->GetPaletteActions(ActionMenuBuilder);

	//@TODO: Avoid this copy
	OutAllActions.Append(ActionMenuBuilder);
}

void SSoundCuePalette::OnGlobalClassViewerFilterModified()
{
	GetMutableDefault<USoundCueGraphSchema>()->UpdateSoundNodeList(FilterData);
	GraphActionMenu->RefreshAllActions(true);
}
