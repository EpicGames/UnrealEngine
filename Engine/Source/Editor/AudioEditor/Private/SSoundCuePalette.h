// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "SGraphPalette.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

struct FGraphActionListBuilderBase;
class IClassViewerFilter;
class FClassViewerFilterFuncs;
class FClassViewerInitializationOptions;

//////////////////////////////////////////////////////////////////////////

class SSoundCuePalette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS( SSoundCuePalette ) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SSoundCuePalette() override;
	
	/** Internal data used to facilitate sound node filtering */
	struct FSoundNodeFilterData
	{	
		TSharedPtr<FClassViewerInitializationOptions> InitOptions;
		TSharedPtr<IClassViewerFilter> ClassFilter;
		TSharedPtr<FClassViewerFilterFuncs> FilterFuncs;
	};

protected:
	/** Callback used to populate all actions list in SGraphActionMenu */
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

	/** Callback when the class viewer filter has been modified */
	void OnGlobalClassViewerFilterModified();
	
	FSoundNodeFilterData FilterData;
};
