// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSoundSubmixActionMenu.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "SGraphActionMenu.h"
#include "SlotBase.h"
#include "SoundSubmixGraph/SoundSubmixGraphSchema.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SSoundSubmixActionMenu"

void SSoundSubmixActionMenuItem::Construct(const FArguments& InArgs, TSharedPtr<FEdGraphSchemaAction> InAction, TWeakPtr<SSoundSubmixActionMenu> InOwner)
{
	check(InAction.IsValid());

	this->Owner = InOwner;

	bool bIsNewSoundSubmix = false;
	if (InAction->GetTypeId() == FSoundSubmixGraphSchemaAction_NewNode::StaticGetTypeId())
	{
		bIsNewSoundSubmix = true;
	}

	// The new sound class widget requires 2 lines as it has a text entry box also.	
	if( !bIsNewSoundSubmix )
	{
		this->ChildSlot
			[
				SNew(SHorizontalBox)
				.ToolTipText(InAction->GetTooltipDescription())
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
					.Text(InAction->GetMenuDescription())
					.HighlightText(InArgs._HighlightText)
				]
			];
	}
	else
	{
		TSharedRef<SWidget> NewSoundSubmixWidget = CreateNewSoundSubmixWidget(InAction->GetMenuDescription(), InAction->GetTooltipDescription(), FCoreStyle::GetDefaultFontStyle("Regular", 9), InAction);
		// Promote requires 2 'slots'
		this->ChildSlot
			[
				NewSoundSubmixWidget
			];
	}
}

TSharedRef<SWidget> SSoundSubmixActionMenuItem::CreateNewSoundSubmixWidget( const FText& DisplayText, const FText& InToolTip, const FSlateFontInfo& NameFont, TSharedPtr<FEdGraphSchemaAction>& InAction )
{
	FString SubmixName;
	FSoundSubmixGraphSchemaAction_NewNode* Action = static_cast<FSoundSubmixGraphSchemaAction_NewNode*>(InAction.Get());
	if (Action)
	{
		SubmixName = Action->NewSoundSubmixName;
	}

	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
			[
				SNew(STextBlock)
				.Text(DisplayText)
				.Font(NameFont)
				.ToolTipText(InToolTip)
			]
		+SVerticalBox::Slot()
			.AutoHeight()
			[			
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(3,0))
				.VAlign(VAlign_Center) 
				[
					SNew(SEditableTextBox)
					.Text(FText::FromString(SubmixName))
					.ToolTipText(InToolTip)
					.OnTextCommitted(this, &SSoundSubmixActionMenuItem::OnNewSoundSubmixNameEntered, InAction)
					.OnTextChanged(this, &SSoundSubmixActionMenuItem::OnNewSoundSubmixNameChanged, InAction)
					.SelectAllTextWhenFocused(true)
					.RevertTextOnEscape(true)
				]
			];

}

void SSoundSubmixActionMenuItem::OnNewSoundSubmixNameChanged( const FText& NewText, TSharedPtr<FEdGraphSchemaAction> InAction )
{
	FSoundSubmixGraphSchemaAction_NewNode* Action = static_cast<FSoundSubmixGraphSchemaAction_NewNode*>(InAction.Get());
	Action->NewSoundSubmixName = NewText.ToString();
}

void SSoundSubmixActionMenuItem::OnNewSoundSubmixNameEntered( const FText& NewText, ETextCommit::Type CommitInfo, TSharedPtr<FEdGraphSchemaAction> InAction )
{
	// Do nothing if we aborted
	if (CommitInfo != ETextCommit::OnEnter)
	{		
		return;
	}

	FSoundSubmixGraphSchemaAction_NewNode* Action = static_cast<FSoundSubmixGraphSchemaAction_NewNode*>(InAction.Get());
	Action->NewSoundSubmixName = *NewText.ToString();

	TArray< TSharedPtr<FEdGraphSchemaAction> > ActionList;
	ActionList.Add( InAction );

	Owner.Pin()->OnActionSelected(ActionList, ESelectInfo::OnKeyPress);
}

///////////////////////////////////////////////

SSoundSubmixActionMenu::~SSoundSubmixActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
}

void SSoundSubmixActionMenu::Construct( const FArguments& InArgs )
{
	this->GraphObj = InArgs._GraphObj;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;

	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5)
		[
			SNew(SBox)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionSelected(this, &SSoundSubmixActionMenu::OnActionSelected)
				.OnCreateWidgetForAction( SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(this, &SSoundSubmixActionMenu::OnCreateWidgetForAction) )
				.OnCollectAllActions(this, &SSoundSubmixActionMenu::CollectAllActions)
				.AutoExpandActionMenu(bAutoExpandActionMenu)
				.ShowFilterTextBox(false)
			]
		]
	);
}

void SSoundSubmixActionMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	// Build up the context object
	FGraphContextMenuBuilder ContextMenuBuilder(GraphObj);
	if (DraggedFromPins.Num() > 0)
	{
		ContextMenuBuilder.FromPin = DraggedFromPins[0];
	}

	// Determine all possible actions
	GraphObj->GetSchema()->GetGraphContextActions(ContextMenuBuilder);

	// Copy the added options back to the main list
	OutAllActions.Append(ContextMenuBuilder);
}

TSharedRef<SWidget> SSoundSubmixActionMenu::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SSoundSubmixActionMenuItem, InCreateData->Action, SharedThis(this))
		.HighlightText(InCreateData->HighlightText);
}


void SSoundSubmixActionMenu::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType )
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		if ( GraphObj != NULL )
		{
			for ( int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++ )
			{
				TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

				if ( CurrentAction.IsValid() )
				{
					FSlateApplication::Get().DismissAllMenus();

					CurrentAction->PerformAction(GraphObj, DraggedFromPins, NewNodePosition);
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
