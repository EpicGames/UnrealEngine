// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class SGraphActionMenu;
class SWidget;
class UEdGraph;
class UEdGraphPin;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;
struct FSlateFontInfo;

/** Widget for displaying a single item  */
class SSoundSubmixActionMenuItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSoundSubmixActionMenuItem )
		{}

		SLATE_ATTRIBUTE( FText, HighlightText )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FEdGraphSchemaAction> InAction, TWeakPtr<class SSoundSubmixActionMenu> InOwner);

private:
	/* Create the new sound submix widget */
	TSharedRef<SWidget> CreateNewSoundSubmixWidget( const FText& DisplayText, const FText& ToolTip, const FSlateFontInfo& NameFont, TSharedPtr<FEdGraphSchemaAction>& InAction );

	/** Called when confirming name for new sound submix */
	void OnNewSoundSubmixNameEntered(const FText& NewText, ETextCommit::Type CommitInfo, TSharedPtr<FEdGraphSchemaAction> InAction );

	/** Called when text is changed for a new sound submix name */
	void OnNewSoundSubmixNameChanged(const FText& NewText,  TSharedPtr<FEdGraphSchemaAction> InAction );

	TWeakPtr<class SSoundSubmixActionMenu> Owner;
};

/////////////////////////////////////////////////////////////////////////////////////////////////

class SSoundSubmixActionMenu : public SBorder
{
public:
	SLATE_BEGIN_ARGS( SSoundSubmixActionMenu )
		: _GraphObj( static_cast<UEdGraph*>(NULL) )
		, _NewNodePosition( FVector2D::ZeroVector )
		, _AutoExpandActionMenu( true )
		{}

		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( FVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	~SSoundSubmixActionMenu();

protected:
	UEdGraph* GraphObj;
	TArray<UEdGraphPin*> DraggedFromPins;
	FDeprecateSlateVector2D NewNodePosition;
	bool bAutoExpandActionMenu;

	SGraphEditor::FActionMenuClosed OnClosedCallback;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	void OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType );

	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);

	/** Callback used to populate all actions list in SGraphActionMenu */
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);

	friend class SSoundSubmixActionMenuItem;
};

