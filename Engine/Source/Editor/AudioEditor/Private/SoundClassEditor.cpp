// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundClassEditor.h"

#include "AssetToolsModule.h"
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#include "AudioEditorModule.h"
#include "ContentBrowserModule.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Factories/SoundClassFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SFindInSoundClassGraph.h"
#include "SSoundClassActionMenu.h"
#include "ScopedTransaction.h"
#include "Sound/SoundClass.h"
#include "SoundClassEditorCommands.h"
#include "SoundClassGraph/SoundClassGraph.h"
#include "SoundClassGraph/SoundClassGraphNode.h"
#include "SoundClassGraph/SoundClassGraphSchema.h"
#include "Styling/AppStyle.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SoundClassEditor"
DEFINE_LOG_CATEGORY_STATIC(LogSoundClassEditor, Log, All);

const FName FSoundClassEditor::GraphCanvasTabId(TEXT("SoundClassEditor_GraphCanvas"));
const FName FSoundClassEditor::PropertiesTabId(TEXT("SoundClassEditor_Properties"));
const FName FSoundClassEditor::FindInSoundClassGraphTabId(TEXT("SoundClassEditor_Find"));


void FSoundClassEditor::ExtendToolbar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Audition");
			{
				ToolbarBuilder.AddToolBarButton(FSoundClassEditorCommands::Get().ToggleMute);
				ToolbarBuilder.AddToolBarButton(FSoundClassEditorCommands::Get().ToggleSolo);
			}
			ToolbarBuilder.EndSection();
		})
	);

	AddToolbarExtender(ToolbarExtender);

	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
	AddToolbarExtender(AudioEditorModule->GetSoundClassToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

//////////////////////////////////////////////////////////////////////////
// FSoundClassEditor

void FSoundClassEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SoundClassEditor", "Sound Class Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(GraphCanvasTabId, FOnSpawnTab::CreateSP(this, &FSoundClassEditor::SpawnTab_GraphCanvas))
		.SetDisplayName(LOCTEXT("GraphCanvasTab", "Graph"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FSoundClassEditor::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	
	InTabManager->RegisterTabSpawner(FindInSoundClassGraphTabId, FOnSpawnTab::CreateSP(this, &FSoundClassEditor::SpawnTab_Find))
		.SetDisplayName(LOCTEXT("FindTab", "Find in Sound Class Graph"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults"));
}

void FSoundClassEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(GraphCanvasTabId);
	InTabManager->UnregisterTabSpawner(PropertiesTabId);
	InTabManager->UnregisterTabSpawner(FindInSoundClassGraphTabId);
}

void FSoundClassEditor::InitSoundClassEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit )
{
	SoundClass = CastChecked<USoundClass>(ObjectToEdit);

	while (SoundClass->ParentClass)
	{
		SoundClass = SoundClass->ParentClass;
	}

	// Support undo/redo
	SoundClass->SetFlags(RF_Transactional);

	GEditor->RegisterForUndo(this);
	
	FSoundClassEditorCommands::Register();
	BindCommands();
	
	if( !SoundClass->SoundClassGraph )
	{
		USoundClassGraph* SoundClassGraph = CastChecked<USoundClassGraph>(FBlueprintEditorUtils::CreateNewGraph(SoundClass, NAME_None, USoundClassGraph::StaticClass(), USoundClassGraphSchema::StaticClass()));
		SoundClassGraph->SetRootSoundClass(SoundClass);

		SoundClass->SoundClassGraph = SoundClassGraph;
	}

	CastChecked<USoundClassGraph>(SoundClass->SoundClassGraph)->RebuildGraph();

	CreateInternalWidgets();

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_SoundClassEditor_Layout_v3" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->AddTab(PropertiesTabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.8f)
				->AddTab(GraphCanvasTabId, ETabState::OpenedTab)
			)
		)
	);
	
	FindWidget = SNew(SFindInSoundClassGraph, GraphEditor)
		.OnSearchResultActivated(this, &FSoundClassEditor::OnSearchResultActivated);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, TEXT("SoundClassEditorApp"), StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, SoundClass);

	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );
	AddMenuExtender(AudioEditorModule->GetSoundClassMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ExtendToolbar();
	RegenerateMenusAndToolbars();
			
#if ENABLE_AUDIO_DEBUG
	if (GEditor->GetAudioDeviceManager())
	{
		Debugger = &GEditor->GetAudioDeviceManager()->GetDebugger();
	}	
#endif

	bool bFoundClass = false;
	GraphEditor->SelectAllNodes();
	for (UObject* SelectedNode : GraphEditor->GetSelectedNodes())
	{
		USoundClassGraphNode* GraphNode = CastChecked<USoundClassGraphNode>(SelectedNode);
		if (GraphNode->SoundClass == ObjectToEdit)
		{
			GraphEditor->ClearSelectionSet();
			GraphEditor->SetNodeSelection(GraphNode, true);
			GraphEditor->ZoomToFit(true);
			DetailsView->SetObject(ObjectToEdit);
			bFoundClass = true;
			break;
		}
	}
	
	if (!bFoundClass)
	{
		GraphEditor->ClearSelectionSet();
		
		const FText NotificationErrorText = FText::Format(LOCTEXT("SoundClassMalformed", "SoundClass {0} has a parent, but this parent does not have it as child!"), FText::AsCultureInvariant(ObjectToEdit->GetName()));
		FNotificationInfo Info(NotificationErrorText);
		Info.ExpireDuration = 3.0f;
		Info.bFireAndForget = true;
		Info.Image = FAppStyle::Get().GetBrush("Icons.Error");
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FSoundClassEditor::BindCommands()
{
	ToolkitCommands->MapAction(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &FSoundClassEditor::UndoGraphAction));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &FSoundClassEditor::RedoGraphAction));

	const FSoundClassEditorCommands& Commands = FSoundClassEditorCommands::Get();
	
	ToolkitCommands->MapAction(
		Commands.ToggleSolo,
		FExecuteAction::CreateSP(this, &FSoundClassEditor::ToggleSolo),
		FCanExecuteAction::CreateSP(this, &FSoundClassEditor::CanExcuteToggleSolo),
		FIsActionChecked::CreateSP(this, &FSoundClassEditor::IsSoloToggled));

	ToolkitCommands->MapAction(
		Commands.ToggleMute,
		FExecuteAction::CreateSP(this, &FSoundClassEditor::ToggleMute),
		FCanExecuteAction::CreateSP(this, &FSoundClassEditor::CanExcuteToggleMute),
		FIsActionChecked::CreateSP(this, &FSoundClassEditor::IsMuteToggled));
	
	ToolkitCommands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, 
		FExecuteAction::CreateSP(this, &FSoundClassEditor::FindInContentBrowser_Execute),
		FCanExecuteAction::CreateSP(this, &FSoundClassEditor::CanFindInContentBrowser));
	
	ToolkitCommands->MapAction(FSoundClassEditorCommands::Get().ViewReferences,
		FExecuteAction::CreateSP(this, &FSoundClassEditor::OnLaunchReferenceViewer));
	
	ToolkitCommands->MapAction(FSoundClassEditorCommands::Get().FindInSoundClassGraph,
		FExecuteAction::CreateSP(this, &FSoundClassEditor::ShowFindInSoundClassGraph));
}

FSoundClassEditor::FSoundClassEditor()
	: SoundClass(nullptr)
#if ENABLE_AUDIO_DEBUG
	, Debugger(nullptr)
#endif
{
}

FSoundClassEditor::~FSoundClassEditor()
{
	GEditor->UnregisterForUndo( this );
	DetailsView.Reset();
}

void FSoundClassEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(SoundClass);
}

TSharedRef<SDockTab> FSoundClassEditor::SpawnTab_GraphCanvas(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == GraphCanvasTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("GraphCanvasTitle", "Graph"))
		[
			GraphEditor.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FSoundClassEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PropertiesTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("SoundClassPropertiesTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];
	
	return SpawnedTab;
}

TSharedRef<SDockTab> FSoundClassEditor::SpawnTab_Find(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FindInSoundClassGraphTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("SoundClassFindTitle", "Find Results")) 
		[
			FindWidget.ToSharedRef()
		];
	
	return SpawnedTab;
}

FName FSoundClassEditor::GetToolkitFName() const
{
	return FName("SoundClassEditor");
}

FText FSoundClassEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Sound Class Editor");
}

FString FSoundClassEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT( "WorldCentricTabPrefix", "Sound Class " ).ToString();
}

FLinearColor FSoundClassEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.3f, 0.2f, 0.5f, 0.5f );
}

void FSoundClassEditor::CreateInternalWidgets()
{
	GraphEditor = CreateGraphEditorWidget();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject( SoundClass );
}

TSharedRef<SGraphEditor> FSoundClassEditor::CreateGraphEditorWidget()
{
	if ( !GraphEditorCommands.IsValid() )
	{
		GraphEditorCommands = MakeShareable( new FUICommandList );

		// Editing commands
		GraphEditorCommands->MapAction( FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP( this, &FSoundClassEditor::SelectAllNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundClassEditor::CanSelectAllNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP( this, &FSoundClassEditor::RemoveSelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundClassEditor::CanRemoveNodes )
			);
		
		// Node commands
		GraphEditorCommands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, 
			FExecuteAction::CreateSP(this, &FSoundClassEditor::FindInContentBrowser_Execute),
			FCanExecuteAction::CreateSP(this, &FSoundClassEditor::CanFindInContentBrowser));
	
		GraphEditorCommands->MapAction(FSoundClassEditorCommands::Get().ViewReferences,
			FExecuteAction::CreateSP(this, &FSoundClassEditor::OnLaunchReferenceViewer));
	}

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_SoundClass", "SOUND CLASS");

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FSoundClassEditor::OnSelectedNodesChanged);
	InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FSoundClassEditor::OnCreateGraphActionMenu);

	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(AppearanceInfo)
		.GraphToEdit(SoundClass->SoundClassGraph)
		.GraphEvents(InEvents)
		.ShowGraphStateOverlay(false);
}

void FSoundClassEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	TArray<UObject*> Selection;

	if(NewSelection.Num())
	{
		for(TSet<class UObject*>::TConstIterator SetIt(NewSelection);SetIt;++SetIt)
		{
			USoundClassGraphNode* GraphNode = CastChecked<USoundClassGraphNode>(*SetIt);
			Selection.Add(GraphNode->SoundClass);
		}
		DetailsView->SetObjects(Selection);
	}
	else
	{
		DetailsView->SetObject(SoundClass);
	}
}

FActionMenuContent FSoundClassEditor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	TSharedRef<SSoundClassActionMenu> ActionMenu = 
		SNew(SSoundClassActionMenu)
		.GraphObj(InGraph)
		.NewNodePosition(FDeprecateSlateVector2D(InNodePosition))
		.DraggedFromPins(InDraggedPins)
		.AutoExpandActionMenu(bAutoExpand)
		.OnClosedCallback(InOnMenuClosed);

	return FActionMenuContent( ActionMenu, ActionMenu );
}

void FSoundClassEditor::SelectAllNodes()
{
	GraphEditor->SelectAllNodes();
}

bool FSoundClassEditor::CanSelectAllNodes() const
{
	return true;
}

void FSoundClassEditor::RemoveSelectedNodes()
{
	const FScopedTransaction Transaction( LOCTEXT("SoundClassEditorRemoveSelectedNode", "Sound Class Editor: Remove Selected SoundClasses from editor") );

	CastChecked<USoundClassGraph>(SoundClass->SoundClassGraph)->RecursivelyRemoveNodes(GraphEditor->GetSelectedNodes());

	GraphEditor->ClearSelectionSet();
}

bool FSoundClassEditor::CanRemoveNodes() const
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		USoundClassGraphNode* Node = Cast<USoundClassGraphNode>(*NodeIt);

		if (Node && Node->CanUserDeleteNode())
		{
			return true;
		}
	}

	return false;
}

void FSoundClassEditor::UndoGraphAction()
{
	GEditor->UndoTransaction();
}

void FSoundClassEditor::RedoGraphAction()
{
	// Clear selection, to avoid holding refs to nodes that go away
	GraphEditor->ClearSelectionSet();

	GEditor->RedoTransaction();
}

void FSoundClassEditor::ToggleSolo()
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		Debugger->ToggleSoloSoundClass(SoundClass->GetFName());
	}
#endif
}

bool FSoundClassEditor::CanExcuteToggleSolo() const
{
#if ENABLE_AUDIO_DEBUG
	// Enable solo if we are in PIE and not Muted 
	if (Debugger)
	{
		bool bIsInPIE = (GEditor != nullptr && GEditor->PlayWorld != nullptr) || GIsPlayInEditorWorld;
		if (bIsInPIE) 
		{
			return !Debugger->IsMuteSoundClass(SoundClass->GetFName());
		}
	}
#endif
	return false;
}

bool FSoundClassEditor::IsSoloToggled() const
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		return Debugger->IsSoloSoundClass(SoundClass->GetFName());
	}
#endif	
	return false;
}

void FSoundClassEditor::ToggleMute()
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		Debugger->ToggleMuteSoundClass(SoundClass->GetFName());
	}
#endif
}

bool FSoundClassEditor::CanExcuteToggleMute() const
{
#if ENABLE_AUDIO_DEBUG
	// Enable mute if we are in PIE and not Soloed
	if (Debugger)
	{
		bool bIsInPIE = (GEditor != nullptr && GEditor->PlayWorld != nullptr) || GIsPlayInEditorWorld;
		if (bIsInPIE)
		{
			return !Debugger->IsSoloSoundClass(SoundClass->GetFName());
		}
	}
#endif
	return false;
}

bool FSoundClassEditor::IsMuteToggled() const
{	
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		return Debugger->IsMuteSoundClass(SoundClass->GetFName());
	}
#endif
	return false;
}

void FSoundClassEditor::FindInContentBrowser_Execute()
{
	if (GraphEditor)
	{
		const FGraphPanelSelectionSet& CurrentSelection = GraphEditor->GetSelectedNodes();
		if (CurrentSelection.Num() > 0)
		{
			TArray<FAssetData> AssetDatas;
			AssetDatas.Reserve(CurrentSelection.Num());
			for (UObject* Node : CurrentSelection)
			{
				USoundClassGraphNode* GraphNode = CastChecked<USoundClassGraphNode>(Node);
				AssetDatas.Emplace(GraphNode->SoundClass);
			}
		
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(AssetDatas);
			return;
		}
	}
	
	// If no node selected or impossible to get to the graph, revert to basic toolkit behavior (find "opened" asset)
	FAssetEditorToolkit::FindInContentBrowser_Execute();
}

void FSoundClassEditor::OnLaunchReferenceViewer() const
{
	TArray<FAssetIdentifier> Assets;
	
	const FGraphPanelSelectionSet& CurrentSelection = GraphEditor->GetSelectedNodes();
	if (CurrentSelection.Num() > 0)
	{
		Assets.Reserve(CurrentSelection.Num());
		for (UObject* Node : CurrentSelection)
		{
			USoundClassGraphNode* GraphNode = CastChecked<USoundClassGraphNode>(Node);
			const FName AssetName(GraphNode->SoundClass->GetPackage()->GetFName());
			Assets.Add(FAssetIdentifier(AssetName));
		}
		
	}
	else
	{
		Assets.Add(FAssetIdentifier(SoundClass->GetPackage()->GetFName()));
	}
	
	FEditorDelegates::OnOpenReferenceViewer.Broadcast(Assets, FReferenceViewerParams());
}

void FSoundClassEditor::ShowFindInSoundClassGraph()
{
	TabManager->TryInvokeTab(FindInSoundClassGraphTabId);
	if (SFindInSoundClassGraph* Widget = FindWidget.Get())
	{
		Widget->FocusForUse();
	}
}

void FSoundClassEditor::OnSearchResultActivated(USoundClassGraphNode* Node)
{
	GraphEditor->ClearSelectionSet();
	
	constexpr bool bSelect = true;
	GraphEditor->SetNodeSelection(Node, bSelect);
	
	constexpr bool bOnlySelection = true;
	GraphEditor->ZoomToFit(bOnlySelection);
	
	DetailsView->SetObject(Node->SoundClass);
}

void FSoundClassEditor::CreateSoundClass(UEdGraphPin* FromPin, const FVector2D& Location, const FString& Name)
{
	// If we have a valid name
	if (!Name.IsEmpty() && Name != SoundClass->GetName())
	{
		// Derive new package path from existing asset's path
		FString PackagePath = SoundClass->GetPathName();
		FString AssetName = FString::Printf(TEXT("/%s.%s"), *SoundClass->GetName(), *SoundClass->GetName());
		PackagePath.RemoveFromEnd(AssetName);

		// Create a sound class factory to create a new sound class
		USoundClassFactory* SoundClassFactory = NewObject<USoundClassFactory>();

		// Load asset tools to create the asset properly
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		USoundClass* NewSoundClass = Cast<USoundClass>(AssetToolsModule.Get().CreateAsset(Name, PackagePath, USoundClass::StaticClass(), SoundClassFactory, FName("SoundClassEditorNewAsset")));

		// This may fail if the asset has same name as existing asset, etc.
		if (NewSoundClass)
		{
			CastChecked<USoundClassGraph>(SoundClass->SoundClassGraph)->AddNewSoundClass(FromPin, NewSoundClass, Location.X, Location.Y);

			NewSoundClass->PostEditChange();
			NewSoundClass->MarkPackageDirty();
		}
	}
}

void FSoundClassEditor::PostUndo(bool bSuccess)
{
	GraphEditor->ClearSelectionSet();
	GraphEditor->NotifyGraphChanged();

	FSlateApplication::Get().DismissAllMenus();
}

#undef LOCTEXT_NAMESPACE
