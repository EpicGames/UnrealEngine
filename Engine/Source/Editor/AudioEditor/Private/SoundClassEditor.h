// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "GraphEditor.h"
#include "ISoundClassEditor.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailsView;
class UEdGraph;
class USoundClass;
class USoundClassGraphNode;
class SFindInSoundClassGraph;

namespace Audio
{
	class FAudioDebugger;
}

//////////////////////////////////////////////////////////////////////////
// FSoundClassEditor

class FSoundClassEditor : public ISoundClassEditor, public FGCObject, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( FSoundClassEditor )
	{
	}

	SLATE_END_ARGS()

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified sound class object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The sound class to edit
	 */
	void InitSoundClassEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit );

	FSoundClassEditor();
	virtual ~FSoundClassEditor();

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSoundClassEditor");
	}

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** ISoundClassEditor interface */
	virtual void CreateSoundClass(class UEdGraphPin* FromPin, const FVector2D& Location, const FString& Name) override;

	/** FEditorUndoClient Interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

private:
	TSharedRef<SDockTab> SpawnTab_GraphCanvas(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Find(const FSpawnTabArgs& Args);

private:
	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Create new graph editor widget */
	TSharedRef<SGraphEditor> CreateGraphEditorWidget();

	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);

	/** Called to create context menu when right-clicking on graph */
	FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Select every node in the graph */
	void SelectAllNodes();
	/** Whether we can select every node */
	bool CanSelectAllNodes() const;

	/** Remove the currently selected nodes from editor view*/
	void RemoveSelectedNodes();
	/** Whether we are able to remove the currently selected nodes */
	bool CanRemoveNodes() const;

	/** Called to undo the last action */
	void UndoGraphAction();
	/** Called to redo the last undone action */
	void RedoGraphAction();

	/** Toggle solo (Soloing means we just hear this) */
	void ToggleSolo();
	bool CanExcuteToggleSolo() const;
	bool IsSoloToggled() const;

	/** Toggle mute (Muting means this will be explicitly silenced)*/
	void ToggleMute();
	bool CanExcuteToggleMute() const;
	bool IsMuteToggled() const;
	
	virtual void FindInContentBrowser_Execute() override;
	void OnLaunchReferenceViewer() const;
	
	/** Show and focus the Find in Sound Class Graph tab. */
	void ShowFindInSoundClassGraph();

private:
	/** The SoundClass asset being inspected */
	TObjectPtr<USoundClass> SoundClass;

	/** Graph Editor */
	TSharedPtr<SGraphEditor> GraphEditor;

	/** Property View */
	TSharedPtr<class IDetailsView> DetailsView;
	
	/** Find in Sound Class Graph widget. */
	TSharedPtr<SFindInSoundClassGraph> FindWidget;

	/** Command list for this editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

#if ENABLE_AUDIO_DEBUG
	/** Cache the audio debugger instance */
	Audio::FAudioDebugger* Debugger;
#endif

	/**	The tab ids for all the tabs used */
	static const FName GraphCanvasTabId;
	static const FName PropertiesTabId;
	static const FName FindInSoundClassGraphTabId;

	/** Bind to commands. */
	void BindCommands();

	/** Helper function to grow the toolbar and add custom buttons */
	void ExtendToolbar();
	
	/** Select & zoom to result */
	void OnSearchResultActivated(USoundClassGraphNode* Node);
};
