// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundClassEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "SoundClassEditorCommands"

void FSoundClassEditorCommands::RegisterCommands()
{
	UI_COMMAND(ToggleSolo, "Solo", "Toggles Soloing this sound class", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::S));
	UI_COMMAND(ToggleMute, "Mute", "Toggles Muting this sound class", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::M));
	UI_COMMAND(ViewReferences, "Reference Viewer...", "Launches the reference viewer showing the selected asset references", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift | EModifierKey::Alt, EKeys::R));
	UI_COMMAND(FindInSoundClassGraph, "Find in Sound Class Graph", "Find a Sound Class within this graph.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
}

#undef LOCTEXT_NAMESPACE
