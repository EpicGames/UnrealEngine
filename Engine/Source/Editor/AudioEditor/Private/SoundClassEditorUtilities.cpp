// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundClassEditorUtilities.h"

#include "EdGraph/EdGraph.h"
#include "HAL/PlatformCrt.h"
#include "ISoundClassEditor.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundClass.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/ToolkitManager.h"

class IToolkit;

void FSoundClassEditorUtilities::CreateSoundClass(const class UEdGraph* Graph, class UEdGraphPin* FromPin, const FVector2D& Location, FString Name)
{
	check(Graph);

	// Cast outer to SoundClass
	USoundClass* SoundClass = CastChecked<USoundClass>(Graph->GetOuter());

	if (SoundClass != NULL)
	{
		TSharedPtr<ISoundClassEditor> SoundClassEditor;
		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(SoundClass);
		if (FoundAssetEditor.IsValid())
		{
			SoundClassEditor = StaticCastSharedPtr<ISoundClassEditor>(FoundAssetEditor);
			SoundClassEditor->CreateSoundClass(FromPin, Location, Name);
		}
	}
}
