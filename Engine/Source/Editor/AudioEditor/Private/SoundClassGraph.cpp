// Copyright Epic Games, Inc. All Rights Reserved.

/////////////////////////////////////////////////////
// USoundClassGraph

#include "SoundClassGraph/SoundClassGraph.h"

#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundClass.h"
#include "SoundClassGraph/SoundClassGraphNode.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundClassGraph)

class FSoundClassAudioEditor : public ISoundClassAudioEditor
{
public:
	void RefreshGraphLinks(UEdGraph* SoundClassGraph) override
	{
		CastChecked<USoundClassGraph>(SoundClassGraph)->RefreshGraphLinks();
	}
};

USoundClassGraph::USoundClassGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RootSoundClass(NULL)
{
	if (!USoundClass::GetSoundClassAudioEditor().IsValid())
	{
		USoundClass::SetSoundClassAudioEditor(TSharedPtr<ISoundClassAudioEditor>(new FSoundClassAudioEditor()));
	}
}

void USoundClassGraph::SetRootSoundClass(USoundClass* InSoundClass)
{
	RootSoundClass = InSoundClass;
}

USoundClass* USoundClassGraph::GetRootSoundClass() const
{
	return RootSoundClass;
}

void USoundClassGraph::RebuildGraph()
{
	check(RootSoundClass);

	// Don't allow initial graph rebuild to affect package dirty state; remember current state...
	UPackage* Package = GetOutermost();
	const bool bIsDirty = Package->IsDirty();

	Modify();

	RemoveAllNodes();

	ConstructNodes(RootSoundClass, 0, 0);

	NotifyGraphChanged();

	// ...and restore it
	Package->SetDirtyFlag(bIsDirty);
}

void USoundClassGraph::AddDroppedSoundClasses(const TArray<USoundClass*>& SoundClasses, int32 NodePosX, int32 NodePosY)
{
	Modify();

	for (int32 ClassIndex = 0; ClassIndex < SoundClasses.Num(); ClassIndex++)
	{
		NodePosY += ConstructNodes(SoundClasses[ClassIndex], NodePosX, NodePosY);
	}

	NotifyGraphChanged();
}

void USoundClassGraph::AddNewSoundClass(UEdGraphPin* FromPin, class USoundClass* SoundClass, int32 NodePosX, int32 NodePosY, bool bSelectNewNode/* = true*/)
{
	Modify();

	USoundClassGraphNode* GraphNode = CreateNode(SoundClass, NodePosX, NodePosY, bSelectNewNode);
	GraphNode->AutowireNewNode(FromPin);

	NotifyGraphChanged();
}

bool USoundClassGraph::IsClassDisplayed(USoundClass* SoundClass) const
{
	return FindExistingNode(SoundClass) != NULL;
}

void USoundClassGraph::LinkSoundClasses()
{
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		USoundClassGraphNode* Node = CastChecked<USoundClassGraphNode>(Nodes[NodeIndex]);

		if (!Node->CheckRepresentsSoundClass())
		{
			Node->SoundClass->Modify();

			// remove parents of existing children
			for (int32 ChildIndex = 0; ChildIndex < Node->SoundClass->ChildClasses.Num(); ChildIndex++)
			{
				USoundClass* ChildClass = Node->SoundClass->ChildClasses[ChildIndex];

				if (ChildClass)
				{
					ChildClass->Modify();
					ChildClass->ParentClass = NULL;
				}
			}

			Node->SoundClass->ChildClasses.Empty();

			UEdGraphPin* ChildPin = Node->GetChildPin();

			for (int32 ChildIndex = 0; ChildIndex < ChildPin->LinkedTo.Num(); ChildIndex++)
			{
				USoundClassGraphNode* ChildNode = CastChecked<USoundClassGraphNode>(ChildPin->LinkedTo[ChildIndex]->GetOwningNode());
				Node->SoundClass->ChildClasses.Add(ChildNode->SoundClass);
				ChildNode->SoundClass->SetParentClass(Node->SoundClass);
			}

			Node->SoundClass->PostEditChange();
			Node->SoundClass->MarkPackageDirty();
		}
	}

	RootSoundClass->RefreshAllGraphs(true);
}

void USoundClassGraph::RefreshGraphLinks()
{
	Modify();

	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		USoundClassGraphNode* Node = CastChecked<USoundClassGraphNode>(Nodes[NodeIndex]);

		if (!Node->CheckRepresentsSoundClass())
		{
			UEdGraphPin* ChildPin = Node->GetChildPin();

			Node->Modify();

			ChildPin->BreakAllPinLinks();

			if (Node->SoundClass)
			{
				for (int32 ChildIndex = 0; ChildIndex < Node->SoundClass->ChildClasses.Num(); ChildIndex++)
				{
					USoundClass* ChildClass = Node->SoundClass->ChildClasses[ChildIndex];

					if (ChildClass)
					{
						USoundClassGraphNode* ChildNode = FindExistingNode(ChildClass);

						if (!ChildNode)
						{
							// New Child not yet represented on graph
							ConstructNodes(ChildClass, Node->NodePosX + 400, Node->NodePosY);
							ChildNode = FindExistingNode(ChildClass);
						}

						ChildPin->MakeLinkTo(ChildNode->GetParentPin());
					}
				}
			}

			Node->PostEditChange();
		}
	}

	NotifyGraphChanged();
}

void USoundClassGraph::RecursivelyRemoveNodes(const TSet<class UObject*> NodesToRemove)
{
	Modify();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(NodesToRemove); NodeIt; ++NodeIt)
	{
		USoundClassGraphNode* Node = Cast<USoundClassGraphNode>(*NodeIt);

		if (Node && Node->CanUserDeleteNode())
		{
			RecursivelyRemoveNode(Node);
		}
	}

	LinkSoundClasses();
}

int32 USoundClassGraph::ConstructNodes(class USoundClass* SoundClass, int32 NodePosX, int32 NodePosY, bool bSelectNewNode/* = true*/)
{
	check(SoundClass);

	TMap<USoundClass*, int32> ChildCounts;

	RecursivelyGatherChildCounts(SoundClass, ChildCounts);

	USoundClassGraphNode* GraphNode = CreateNode(SoundClass, NodePosX, NodePosY, bSelectNewNode);

	return RecursivelyConstructChildNodes(GraphNode, ChildCounts);
}

int32 USoundClassGraph::RecursivelyGatherChildCounts(USoundClass* ParentClass, TMap<USoundClass*, int32>& OutChildCounts)
{
	int32 ChildSize = 0;

	for (int32 ChildIndex = 0; ChildIndex < ParentClass->ChildClasses.Num(); ChildIndex++)
	{
		if (ParentClass->ChildClasses[ChildIndex])
		{
			ChildSize += RecursivelyGatherChildCounts(ParentClass->ChildClasses[ChildIndex], OutChildCounts);
		}
	}

	if (ChildSize == 0)
	{
		ChildSize = 1;
	}

	OutChildCounts.Add(ParentClass, ChildSize);
	return ChildSize;
}

int32 USoundClassGraph::RecursivelyConstructChildNodes(USoundClassGraphNode* ParentNode, const TMap<USoundClass*, int32>& InChildCounts, bool bSelectNewNode/* = true*/)
{
	const int32 HorizontalSpacing = 400;
	const int32 VerticalSpacing = 100;

	USoundClass* ParentClass = ParentNode->SoundClass;
	int32 TotalChildSizeY = InChildCounts.FindChecked(ParentClass) * VerticalSpacing;
	int32 NodeStartY = ParentNode->NodePosY - (TotalChildSizeY * 0.5f) + (VerticalSpacing * 0.5f);
	int32 NodePosX = ParentNode->NodePosX + HorizontalSpacing;

	for (int32 ChildIndex = 0; ChildIndex < ParentClass->ChildClasses.Num(); ChildIndex++)
	{
		if (ParentClass->ChildClasses[ChildIndex])
		{
			const int32 ChildCount = InChildCounts.FindChecked(ParentClass->ChildClasses[ChildIndex]);
			int32 NodePosY = NodeStartY + (ChildCount * VerticalSpacing * 0.5f) - (VerticalSpacing * 0.5f);
			USoundClassGraphNode* ChildNode = CreateNode(ParentClass->ChildClasses[ChildIndex], NodePosX, NodePosY, bSelectNewNode);
			ParentNode->GetChildPin()->MakeLinkTo(ChildNode->GetParentPin());
			RecursivelyConstructChildNodes(ChildNode, InChildCounts);
			NodeStartY += ChildCount * VerticalSpacing;
		}
	}

	return TotalChildSizeY;
}

void USoundClassGraph::RecursivelyRemoveNode(class USoundClassGraphNode* ParentNode)
{
	UEdGraphPin* ChildPin = ParentNode->GetChildPin();

	for (int32 ChildIndex = ChildPin->LinkedTo.Num() - 1; ChildIndex >= 0; ChildIndex--)
	{
		USoundClassGraphNode* ChildNode = CastChecked<USoundClassGraphNode>(ChildPin->LinkedTo[ChildIndex]->GetOwningNode());
		RecursivelyRemoveNode(ChildNode);
	}

	ParentNode->Modify();
	RemoveNode(ParentNode);
}

void USoundClassGraph::RemoveAllNodes()
{
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		NodesToRemove[NodeIndex]->Modify();
		RemoveNode(NodesToRemove[NodeIndex]);
	}
}

USoundClassGraphNode* USoundClassGraph::CreateNode(USoundClass* SoundClass, int32 NodePosX, int32 NodePosY, bool bSelectNewNode/* = true*/)
{
	USoundClassGraphNode* GraphNode = FindExistingNode(SoundClass);

	if (!GraphNode)
	{
		FGraphNodeCreator<USoundClassGraphNode> NodeCreator(*this);
		GraphNode = NodeCreator.CreateNode(bSelectNewNode);
		GraphNode->SoundClass = SoundClass;
		GraphNode->NodePosX = NodePosX;
		GraphNode->NodePosY = NodePosY;
		NodeCreator.Finalize();
	}
	return GraphNode;
}

USoundClassGraphNode* USoundClassGraph::FindExistingNode(USoundClass* SoundClass) const
{
	USoundClassGraphNode* ExistingNode = NULL;

	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
	{
		USoundClassGraphNode* Node = CastChecked<USoundClassGraphNode>(Nodes[NodeIndex]);
		if (Node->SoundClass == SoundClass)
		{
			ExistingNode = Node;
			break;
		}
	}

	return ExistingNode;
}
