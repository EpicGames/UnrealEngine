// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundClassGraph/SoundClassGraphNode.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundClass.h"
#include "SoundClassGraph/SoundClassGraph.h"
#include "SoundClassGraph/SoundClassGraphSchema.h"
#include "Templates/Casts.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundClassGraphNode)

#define LOCTEXT_NAMESPACE "SoundClassGraphNode"

/////////////////////////////////////////////////////
// USoundClassGraphNode

USoundClassGraphNode::USoundClassGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ChildPin(NULL)
	, ParentPin(NULL)
{
}

bool USoundClassGraphNode::CheckRepresentsSoundClass()
{
	if (!SoundClass)
	{
		return false;
	}

	for (int32 ChildIndex = 0; ChildIndex < ChildPin->LinkedTo.Num(); ChildIndex++)
	{
		USoundClassGraphNode* ChildNode = CastChecked<USoundClassGraphNode>(ChildPin->LinkedTo[ChildIndex]->GetOwningNode());
		if (!SoundClass->ChildClasses.Contains(ChildNode->SoundClass))
		{
			return false;
		}
	}

	for (int32 ChildIndex = 0; ChildIndex < SoundClass->ChildClasses.Num(); ChildIndex++)
	{
		bool bFoundChild = false;
		for (int32 NodeChildIndex = 0; NodeChildIndex < ChildPin->LinkedTo.Num(); NodeChildIndex++)
		{
			USoundClassGraphNode* ChildNode = CastChecked<USoundClassGraphNode>(ChildPin->LinkedTo[NodeChildIndex]->GetOwningNode());
			if (ChildNode->SoundClass == SoundClass->ChildClasses[ChildIndex])
			{
				bFoundChild = true;
				break;
			}
		}

		if (!bFoundChild)
		{
			return false;
		}
	}

	return true;
}

FLinearColor USoundClassGraphNode::GetNodeTitleColor() const
{
	if (SoundClass && SoundClass->PassiveSoundMixModifiers.Num() > 0)
	{
		return FLinearColor::Green;
	}
	else
	{
		return Super::GetNodeTitleColor();
	}
}

void USoundClassGraphNode::AllocateDefaultPins()
{
	check(Pins.Num() == 0);

	ChildPin = CreatePin(EGPD_Output, TEXT("SoundClass"), *LOCTEXT("SoundClassChildren", "Children").ToString());
	ParentPin = CreatePin(EGPD_Input, TEXT("SoundClass"), NAME_None);
}

void USoundClassGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin)
	{
		const USoundClassGraphSchema* Schema = CastChecked<USoundClassGraphSchema>(GetSchema());

		if (FromPin->Direction == EGPD_Input)
		{
			Schema->TryCreateConnection(FromPin, ChildPin);
		}
		else
		{
			Schema->TryCreateConnection(FromPin, ParentPin);
		}
	}
}

bool USoundClassGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(USoundClassGraphSchema::StaticClass());
}

FText USoundClassGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (SoundClass)
	{
		return FText::FromString(SoundClass->GetName());
	}
	else
	{
		return Super::GetNodeTitle(TitleType);
	}
}

bool USoundClassGraphNode::CanUserDeleteNode() const
{
	USoundClassGraph* SoundClassGraph = CastChecked<USoundClassGraph>(GetGraph());

	// Cannot remove the root node from the graph
	return SoundClass != SoundClassGraph->GetRootSoundClass();
}

#undef LOCTEXT_NAMESPACE
