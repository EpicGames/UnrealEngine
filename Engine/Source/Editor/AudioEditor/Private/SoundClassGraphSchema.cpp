// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundClassGraph/SoundClassGraphSchema.h"

#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "ScopedTransaction.h"
#include "SoundClassEditorCommands.h"
#include "Sound/SoundClass.h"
#include "SoundClassEditorUtilities.h"
#include "SoundClassGraph/SoundClassGraph.h"
#include "SoundClassGraph/SoundClassGraphNode.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundClassGraphSchema)

#define LOCTEXT_NAMESPACE "SoundClassSchema"

UEdGraphNode* FSoundClassGraphSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	FSoundClassEditorUtilities::CreateSoundClass(ParentGraph, FromPin, FDeprecateSlateVector2D(Location), NewSoundClassName);
	return NULL;
}

USoundClassGraphSchema::USoundClassGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USoundClassGraphSchema::ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	USoundClassGraphNode* InputNode = CastChecked<USoundClassGraphNode>(InputPin->GetOwningNode());
	USoundClassGraphNode* OutputNode = CastChecked<USoundClassGraphNode>(OutputPin->GetOwningNode());

	return InputNode->SoundClass->RecurseCheckChild( OutputNode->SoundClass );
}

void USoundClassGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const FText Name = LOCTEXT("NewSoundClass", "New Sound Class");
	const FText ToolTip = LOCTEXT("NewSoundClassTooltip", "Create a new sound class");
	
	TSharedPtr<FSoundClassGraphSchemaAction_NewNode> NewAction(new FSoundClassGraphSchemaAction_NewNode(FText::GetEmpty(), Name, ToolTip, 0));

	ContextMenuBuilder.AddAction( NewAction );
}

void USoundClassGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	FToolMenuSection& Section = Menu->AddSection("SoundClassGraphSchemaNodeActions", LOCTEXT("ClassActionsMenuHeader", "SoundClass Actions"));
	Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
	Section.AddMenuEntry(FGenericCommands::Get().Delete);
	
	FToolMenuSection& AssetSection = Menu->AddSection("AssetGraphSchemaNodeActions", LOCTEXT("AssetActionsMenuHeader", "Asset Actions"));
	AssetSection.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);
	AssetSection.AddMenuEntry(FSoundClassEditorCommands::Get().ViewReferences);

	// No Super call so Node comments option is not shown
}

const FPinConnectionResponse USoundClassGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = NULL;
	const UEdGraphPin* OutputPin = NULL;

	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatible", "Directions are not compatible"));
	}

	if (ConnectionCausesLoop(InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop"));
	}

	// Break existing connections on inputs only - multiple output connections are acceptable
	if (InputPin->LinkedTo.Num() > 0)
	{
		ECanCreateConnectionResponse ReplyBreakOutputs;
		if (InputPin == PinA)
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
		}
		else
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
		}
		return FPinConnectionResponse(ReplyBreakOutputs, LOCTEXT("ConnectionReplace", "Replace existing connections"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, FText::GetEmpty());
}

bool USoundClassGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	bool bModified = UEdGraphSchema::TryCreateConnection(PinA, PinB);

	if (bModified)
	{
		CastChecked<USoundClassGraph>(PinA->GetOwningNode()->GetGraph())->LinkSoundClasses();
	}

	return bModified;
}

bool USoundClassGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	return true;
}

FLinearColor USoundClassGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::White;
}

void USoundClassGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	Super::BreakNodeLinks(TargetNode);

	CastChecked<USoundClassGraph>(TargetNode.GetGraph())->LinkSoundClasses();
}

void USoundClassGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links") );

	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);
	
	// if this would notify the node then we need to re-link sound classes
	if (bSendsNodeNotifcation)
	{
		CastChecked<USoundClassGraph>(TargetPin.GetOwningNode()->GetGraph())->LinkSoundClasses();
	}
}

void USoundClassGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link") );
	Super::BreakSinglePinLink(SourcePin, TargetPin);

	CastChecked<USoundClassGraph>(SourcePin->GetOwningNode()->GetGraph())->LinkSoundClasses();
}

void USoundClassGraphSchema::DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	USoundClassGraph* SoundClassGraph = CastChecked<USoundClassGraph>(Graph);

	TArray<USoundClass*> UndisplayedClasses;
	for (int32 AssetIdx = 0; AssetIdx < Assets.Num(); ++AssetIdx)
	{
		USoundClass* SoundClass = Cast<USoundClass>(Assets[AssetIdx].GetAsset());
		if (SoundClass && !SoundClassGraph->IsClassDisplayed(SoundClass))
		{
			UndisplayedClasses.Add(SoundClass);
		}
	}

	if (UndisplayedClasses.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("SoundClassEditorDropClasses", "Sound Class Editor: Drag and Drop Sound Class") );

		SoundClassGraph->AddDroppedSoundClasses(UndisplayedClasses, GraphPosition.X, GraphPosition.Y);
	}
}

void USoundClassGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const USoundClassGraph* SoundClassGraph = CastChecked<const USoundClassGraph>(HoverGraph);
	for (int32 AssetIdx = 0; AssetIdx < Assets.Num(); ++AssetIdx)
	{
		USoundClass* SoundClass = Cast<USoundClass>(Assets[AssetIdx].GetAsset());
		if (SoundClass)
		{
			if (!SoundClassGraph->IsClassDisplayed(SoundClass))
			{
				OutTooltipText = FText::FormatOrdered(LOCTEXT("DropSoundClass", "Add {0} to Graph."), USoundClass::StaticClass()->GetDisplayNameText()).ToString();
				OutOkIcon = true;
				break;
			}
			else if (OutTooltipText.IsEmpty())
			{
				OutTooltipText = FText::FormatOrdered(LOCTEXT("SoundClassInGraph", "{0} already in graph."), USoundClass::StaticClass()->GetDisplayNameText()).ToString();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
