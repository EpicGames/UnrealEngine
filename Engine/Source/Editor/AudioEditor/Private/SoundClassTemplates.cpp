// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundClassTemplates.h"

#include "Components/SynthComponent.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/SoundEffectSubmix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundClassTemplates)

USoundEffectSourcePresetClassTemplate::USoundEffectSourcePresetClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UClassTemplate(ObjectInitializer)
{
	SetGeneratedBaseClass(USoundEffectSourcePreset::StaticClass());
}

USoundEffectSubmixPresetClassTemplate::USoundEffectSubmixPresetClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UClassTemplate(ObjectInitializer)
{
	SetGeneratedBaseClass(USoundEffectSubmixPreset::StaticClass());
}

USynthComponentClassTemplate::USynthComponentClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UClassTemplate(ObjectInitializer)
{
	SetGeneratedBaseClass(USynthComponent::StaticClass());
}
