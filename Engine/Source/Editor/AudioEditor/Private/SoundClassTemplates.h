// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClassTemplateEditorSubsystem.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundClassTemplates.generated.h"

class UObject;


UCLASS()
class USoundEffectSourcePresetClassTemplate : public UClassTemplate
{
	GENERATED_UCLASS_BODY()

public:
	FString GetFilename() const override
	{
		return TEXT("SoundEffectSourceClass");
	}
};

UCLASS()
class USoundEffectSubmixPresetClassTemplate : public UClassTemplate
{
	GENERATED_UCLASS_BODY()

public:
	FString GetFilename() const override
	{
		return TEXT("SoundEffectSubmixClass");
	}
};

UCLASS()
class USynthComponentClassTemplate : public UClassTemplate
{
	GENERATED_UCLASS_BODY()

public:
	FString GetFilename() const override
	{
		return TEXT("SynthComponentClass");
	}
};