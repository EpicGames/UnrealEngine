// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EdGraph/EdGraphNode.h"
#include "EngineAnalytics.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "SoundCueGraph/SoundCueGraphNode_Root.h"
#include "SoundCueGraph/SoundCueGraphSchema.h"
#include "Sound/SoundWave.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundCue.h"
#include "Components/AudioComponent.h"
#include "AudioEditorModule.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "ScopedTransaction.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EdGraphUtilities.h"
#include "SNodePanel.h"
#include "Editor.h"
#include "SoundCueGraphEditorCommands.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Commands/GenericCommands.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "SSoundCuePalette.h"
#include "HAL/PlatformApplicationMisc.h"
#include "AudioDeviceManager.h"
#include "Audio/AudioDebug.h"

#define LOCTEXT_NAMESPACE "SoundCueEditor"

const FName FSoundCueEditor::GraphCanvasTabId( TEXT( "SoundCueEditor_GraphCanvas" ) );
const FName FSoundCueEditor::PropertiesTabId( TEXT( "SoundCueEditor_Properties" ) );
const FName FSoundCueEditor::PaletteTabId( TEXT( "SoundCueEditor_Palette" ) );

FSoundCueEditor::FSoundCueEditor()
	: SoundCue(nullptr)
#if ENABLE_AUDIO_DEBUG
	, Debugger(nullptr)
#endif
{
}

void FSoundCueEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SoundCueEditor", "Sound Cue Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( GraphCanvasTabId, FOnSpawnTab::CreateSP(this, &FSoundCueEditor::SpawnTab_GraphCanvas) )
		.SetDisplayName( LOCTEXT("GraphCanvasTab", "Viewport") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner( PropertiesTabId, FOnSpawnTab::CreateSP(this, &FSoundCueEditor::SpawnTab_Properties) )
		.SetDisplayName( LOCTEXT("DetailsTab", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner( PaletteTabId, FOnSpawnTab::CreateSP(this, &FSoundCueEditor::SpawnTab_Palette) )
		.SetDisplayName( LOCTEXT("PaletteTab", "Palette") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));
}

void FSoundCueEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( GraphCanvasTabId );
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
	InTabManager->UnregisterTabSpawner( PaletteTabId );
}

FSoundCueEditor::~FSoundCueEditor()
{
	// Stop any playing sound cues when the cue editor closes
	UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
	if (PreviewComp && PreviewComp->IsPlaying())
	{
		Stop();
	}

	GEditor->UnregisterForUndo( this );
}

void FSoundCueEditor::InitSoundCueEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit)
{
	SoundCue = CastChecked<USoundCue>(ObjectToEdit);

	// Support undo/redo
	SoundCue->SetFlags(RF_Transactional);
	
	GEditor->RegisterForUndo(this);

	FGraphEditorCommands::Register();
	FSoundCueGraphEditorCommands::Register();

	BindGraphCommands();

	CreateInternalWidgets();

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_SoundCueEditor_Layout_v5")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split(FTabManager::NewSplitter()
			->SetOrientation(Orient_Horizontal)
			->SetSizeCoefficient(0.9f)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.225f)
				->SetHideTabWell(true)
				->AddTab(PropertiesTabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.65f)
				->SetHideTabWell(true)
				->AddTab(GraphCanvasTabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.125f)
				->SetHideTabWell(true)
				->AddTab(PaletteTabId, ETabState::OpenedTab)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TEXT("SoundCueEditorApp"), StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit, false);

	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );
	AddMenuExtender(AudioEditorModule->GetSoundCueMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ExtendToolbar();
	RegenerateMenusAndToolbars();

#if ENABLE_AUDIO_DEBUG
	if (GEditor->GetAudioDeviceManager())
	{
		Debugger = &GEditor->GetAudioDeviceManager()->GetDebugger();
	}	
#endif
	
	// @todo toolkit world centric editing
	/*if(IsWorldCentricAssetEditor())
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		SpawnToolkitTab(GraphCanvasTabId, FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(PropertiesTabId, FString(), EToolkitTabSpot::Details);
	}*/
}

USoundCue* FSoundCueEditor::GetSoundCue() const
{
	return SoundCue;
}

void FSoundCueEditor::SetSelection(TArray<UObject*> SelectedObjects)
{
	if (SoundCueProperties.IsValid())
	{
		SoundCueProperties->SetObjects(SelectedObjects);
	}
}

bool FSoundCueEditor::GetBoundsForSelectedNodes(class FSlateRect& Rect, float Padding )
{
	return SoundCueGraphEditor->GetBoundsForSelectedNodes(Rect, Padding);
}

int32 FSoundCueEditor::GetNumberOfSelectedNodes() const
{
	return SoundCueGraphEditor->GetSelectedNodes().Num();
}

FName FSoundCueEditor::GetToolkitFName() const
{
	return FName("SoundCueEditor");
}

FText FSoundCueEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "SoundCue Editor");
}

FString FSoundCueEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "SoundCue ").ToString();
}

FLinearColor FSoundCueEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

TSharedRef<SDockTab> FSoundCueEditor::SpawnTab_GraphCanvas(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == GraphCanvasTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("SoundCueGraphCanvasTitle", "Viewport"));

	if (SoundCueGraphEditor.IsValid())
	{
		SpawnedTab->SetContent(SoundCueGraphEditor.ToSharedRef());
	}

	return SpawnedTab;
}

TSharedRef<SDockTab> FSoundCueEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == PropertiesTabId );

	return SNew(SDockTab)
		.Label(LOCTEXT("SoundCueDetailsTitle", "Details"))
		[
			SoundCueProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FSoundCueEditor::SpawnTab_Palette(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == PaletteTabId );

	return SNew(SDockTab)
		.Label(LOCTEXT("SoundCuePaletteTitle", "Palette"))
		[
			Palette.ToSharedRef()
		];
}

void FSoundCueEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SoundCue);
}

void FSoundCueEditor::PostUndo(bool bSuccess)
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->ClearSelectionSet();
		SoundCueGraphEditor->NotifyGraphChanged();
		FSlateApplication::Get().DismissAllMenus();
	}

}

void FSoundCueEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FProperty* PropertyThatChanged)
{
	if (SoundCueGraphEditor.IsValid() && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		SoundCueGraphEditor->NotifyGraphChanged();
	}
}

void FSoundCueEditor::CreateInternalWidgets()
{
	SoundCueGraphEditor = CreateGraphEditorWidget();

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	SoundCueProperties = PropertyModule.CreateDetailView(Args);
	SoundCueProperties->SetObject( SoundCue );

	Palette = SNew(SSoundCuePalette);
}

void FSoundCueEditor::ExtendToolbar()
{
	struct Local
{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Debug");
			{
				ToolbarBuilder.AddToolBarButton(FSoundCueGraphEditorCommands::Get().ToggleSolo);

				ToolbarBuilder.AddToolBarButton(FSoundCueGraphEditorCommands::Get().ToggleMute);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("Toolbar");
			{
				ToolbarBuilder.AddToolBarButton(FSoundCueGraphEditorCommands::Get().PlayCue);

				ToolbarBuilder.AddToolBarButton(FSoundCueGraphEditorCommands::Get().PlayNode);

				ToolbarBuilder.AddToolBarButton(FSoundCueGraphEditorCommands::Get().StopCueNode);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar )
		);

	AddToolbarExtender(ToolbarExtender);

	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );
	AddToolbarExtender(AudioEditorModule->GetSoundCueToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FSoundCueEditor::BindGraphCommands()
{
	const FSoundCueGraphEditorCommands& Commands = FSoundCueGraphEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.PlayCue,
		FExecuteAction::CreateSP(this, &FSoundCueEditor::PlayCue));

	ToolkitCommands->MapAction(
		Commands.PlayNode,
		FExecuteAction::CreateSP(this, &FSoundCueEditor::PlayNode),
		FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanPlayNode ));

	ToolkitCommands->MapAction(
		Commands.StopCueNode,
		FExecuteAction::CreateSP(this, &FSoundCueEditor::Stop));

	ToolkitCommands->MapAction(
		Commands.TogglePlayback,
		FExecuteAction::CreateSP(this, &FSoundCueEditor::TogglePlayback));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP( this, &FSoundCueEditor::UndoGraphAction ));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP( this, &FSoundCueEditor::RedoGraphAction ));

	ToolkitCommands->MapAction(
		Commands.ToggleSolo,
		FExecuteAction::CreateSP(this, &FSoundCueEditor::ToggleSolo),
		FCanExecuteAction::CreateSP(this, &FSoundCueEditor::CanExcuteToggleSolo),
		FIsActionChecked::CreateSP(this, &FSoundCueEditor::IsSoloToggled));
		
	ToolkitCommands->MapAction(
		Commands.ToggleMute,
		FExecuteAction::CreateSP(this, &FSoundCueEditor::ToggleMute),
		FCanExecuteAction::CreateSP(this, &FSoundCueEditor::CanExcuteToggleMute),
		FIsActionChecked::CreateSP(this, &FSoundCueEditor::IsMuteToggled));
}

void FSoundCueEditor::PlayCue()
{
	GEditor->PlayPreviewSound(SoundCue);

	SoundCueGraphEditor->RegisterActiveTimer(0.0f, 
	FWidgetActiveTimerDelegate::CreateLambda(
		[](double InCurrentTime, float InDeltaTime)
		{
			UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
			if (PreviewComp && PreviewComp->IsPlaying())
			{
				return EActiveTimerReturnType::Continue;
			}
			else
			{
				return EActiveTimerReturnType::Stop;
			}
		}));
}

void FSoundCueEditor::PlayNode()
{
	// already checked that only one node is selected
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		PlaySingleNode(CastChecked<UEdGraphNode>(*NodeIt));
	}
}

bool FSoundCueEditor::CanPlayNode() const
{
	return GetSelectedNodes().Num() == 1;
}

void FSoundCueEditor::Stop()
{
	GEditor->ResetPreviewAudioComponent();
}

void FSoundCueEditor::TogglePlayback()
{
	UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
	if ( PreviewComp && PreviewComp->IsPlaying() )
	{
		Stop();
	}
	else
	{
		PlayCue();
	}
}

void FSoundCueEditor::ToggleSolo()
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		Debugger->ToggleSoloSoundCue(SoundCue->GetFName());
	}
#endif
}

bool FSoundCueEditor::CanExcuteToggleSolo() const
{
#if ENABLE_AUDIO_DEBUG
	// Allow Solo if Mute is not Toggle on
	if (Debugger)
	{
		return !Debugger->IsMuteSoundCue(SoundCue->GetFName());
	}
#endif
	return false;
}

bool FSoundCueEditor::IsSoloToggled() const
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		return Debugger->IsSoloSoundCue(SoundCue->GetFName());
	}
#endif
	return false;
}

void FSoundCueEditor::ToggleMute()
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		Debugger->ToggleMuteSoundCue(SoundCue->GetFName());
	}
#endif
}

bool FSoundCueEditor::CanExcuteToggleMute() const
{
#if ENABLE_AUDIO_DEBUG
	// Allow Mute if Solo is not Toggle on
	if (Debugger)
	{
		return !Debugger->IsSoloSoundCue(SoundCue->GetFName());
	}
#endif
	return false;
}

bool FSoundCueEditor::IsMuteToggled() const
{
#if ENABLE_AUDIO_DEBUG
	if (Debugger)
	{
		return Debugger->IsMuteSoundCue(SoundCue->GetFName());
	}
#endif
	return false;
}

void FSoundCueEditor::PlaySingleNode(UEdGraphNode* Node)
{
	USoundCueGraphNode* SoundGraphNode = Cast<USoundCueGraphNode>(Node);

	if (SoundGraphNode)
	{
		GEditor->PlayPreviewSound(NULL, SoundGraphNode->SoundNode);
	}
	else
	{
		// must be root node, play the whole cue
		PlayCue();
	}
}

void FSoundCueEditor::SyncInBrowser()
{
	TArray<UObject*> ObjectsToSync;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		USoundCueGraphNode* SelectedNode = Cast<USoundCueGraphNode>(*NodeIt);

		if (SelectedNode)
		{
			USoundNodeWavePlayer* SelectedWave = Cast<USoundNodeWavePlayer>(SelectedNode->SoundNode);
			if (SelectedWave && SelectedWave->GetSoundWave())
			{
				ObjectsToSync.AddUnique(SelectedWave->GetSoundWave());
			}

			USoundNodeDialoguePlayer* SelectedDialogue = Cast<USoundNodeDialoguePlayer>(SelectedNode->SoundNode);
			if (SelectedDialogue && SelectedDialogue->GetDialogueWave())
			{
				ObjectsToSync.AddUnique(SelectedDialogue->GetDialogueWave());
			}
		}
	}

	if (ObjectsToSync.Num() > 0)
	{
		GEditor->SyncBrowserToObjects(ObjectsToSync);
	}
}

bool FSoundCueEditor::CanSyncInBrowser() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		USoundCueGraphNode* SelectedNode = Cast<USoundCueGraphNode>(*NodeIt);

		if (SelectedNode)
		{
			USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>(SelectedNode->SoundNode);

			if (WavePlayer && WavePlayer->GetSoundWave())
			{
				return true;
			}

			USoundNodeDialoguePlayer* SelectedDialogue = Cast<USoundNodeDialoguePlayer>(SelectedNode->SoundNode);
			if (SelectedDialogue && SelectedDialogue->GetDialogueWave())
			{
				return true;
			}
		}
	}
	return false;
}

void FSoundCueEditor::AddInput()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	// Iterator used but should only contain one node
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		USoundCueGraphNode* SelectedNode = Cast<USoundCueGraphNode>(*NodeIt);

		if (SelectedNode)
		{
			SelectedNode->AddInputPin();
			break;
		}
	}
}

bool FSoundCueEditor::CanAddInput() const
{
	return GetSelectedNodes().Num() == 1;
}

void FSoundCueEditor::DeleteInput()
{
	if (SoundCueGraphEditor.IsValid())
	{
		UEdGraphPin* SelectedPin = SoundCueGraphEditor->GetGraphPinForMenu();
		if (ensure(SelectedPin))
		{
			USoundCueGraphNode* SelectedNode = Cast<USoundCueGraphNode>(SelectedPin->GetOwningNode());

			if (SelectedNode && SelectedNode == SelectedPin->GetOwningNode())
			{
				SelectedNode->RemoveInputPin(SelectedPin);
			}
		}
	}
}

bool FSoundCueEditor::CanDeleteInput() const
{
	return true;
}

void FSoundCueEditor::OnCreateComment()
{
	FSoundCueGraphSchemaAction_NewComment CommentAction;
	CommentAction.PerformAction(SoundCue->SoundCueGraph, NULL, SoundCueGraphEditor->GetPasteLocation2f());
}

TSharedRef<SGraphEditor> FSoundCueEditor::CreateGraphEditorWidget()
{
	if ( !GraphEditorCommands.IsValid() )
	{
		GraphEditorCommands = MakeShareable( new FUICommandList );

		GraphEditorCommands->MapAction( FSoundCueGraphEditorCommands::Get().PlayNode,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::PlayNode),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanPlayNode ));

		GraphEditorCommands->MapAction( FSoundCueGraphEditorCommands::Get().BrowserSync,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::SyncInBrowser),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanSyncInBrowser ));

		GraphEditorCommands->MapAction( FSoundCueGraphEditorCommands::Get().AddInput,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::AddInput),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanAddInput ));

		GraphEditorCommands->MapAction( FSoundCueGraphEditorCommands::Get().DeleteInput,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::DeleteInput),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanDeleteInput ));

		// Graph Editor Commands
		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CreateComment,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::OnCreateComment )
			);

		// Editing commands
		GraphEditorCommands->MapAction( FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::SelectAllNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanSelectAllNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::DeleteSelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanDeleteNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::CopySelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanCopyNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::CutSelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanCutNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::PasteNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanPasteNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP( this, &FSoundCueEditor::DuplicateNodes ),
			FCanExecuteAction::CreateSP( this, &FSoundCueEditor::CanDuplicateNodes )
			);

		// Alignment Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnAlignTop)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnAlignMiddle)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnAlignBottom)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnAlignLeft)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnAlignCenter)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnAlignRight)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnStraightenConnections)
		);

		// Distribution Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnDistributeNodesH)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
			FExecuteAction::CreateSP(this, &FSoundCueEditor::OnDistributeNodesV)
		);
	}

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_SoundCue", "SOUND CUE");

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FSoundCueEditor::OnSelectedNodesChanged);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FSoundCueEditor::OnNodeTitleCommitted);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FSoundCueEditor::PlaySingleNode);

	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(AppearanceInfo)
		.GraphToEdit(SoundCue->GetGraph())
		.GraphEvents(InEvents)
		.AutoExpandActionMenu(true)
		.ShowGraphStateOverlay(false);
}

FGraphPanelSelectionSet FSoundCueEditor::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	if (SoundCueGraphEditor.IsValid())
	{
		CurrentSelection = SoundCueGraphEditor->GetSelectedNodes();
	}
	return CurrentSelection;
}

void FSoundCueEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	TArray<UObject*> Selection;

	if(NewSelection.Num())
	{
		for(TSet<class UObject*>::TConstIterator SetIt(NewSelection);SetIt;++SetIt)
		{
			if (Cast<USoundCueGraphNode_Root>(*SetIt))
			{
				Selection.Add(GetSoundCue());
			}
			else if (USoundCueGraphNode* GraphNode = Cast<USoundCueGraphNode>(*SetIt))
			{
				Selection.Add(GraphNode->SoundNode);
			}
			else
			{
				Selection.Add(*SetIt);
			}
		}
		//Selection = NewSelection.Array();
	}
	else
	{
		Selection.Add(GetSoundCue());
	}

	SetSelection(Selection);
}

void FSoundCueEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction( LOCTEXT( "RenameNode", "Rename Node" ) );
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

void FSoundCueEditor::SelectAllNodes()
{
	SoundCueGraphEditor->SelectAllNodes();
}

bool FSoundCueEditor::CanSelectAllNodes() const
{
	return true;
}

void FSoundCueEditor::DeleteSelectedNodes()
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SoundCueEditorDeleteSelectedNode", "Delete Selected Sound Cue Node") );

	SoundCueGraphEditor->GetCurrentGraph()->Modify();

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	SoundCueGraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = CastChecked<UEdGraphNode>(*NodeIt);

		if (Node->CanUserDeleteNode())
		{
			if (USoundCueGraphNode* SoundGraphNode = Cast<USoundCueGraphNode>(Node))
			{
				USoundNode* DelNode = SoundGraphNode->SoundNode;

				FBlueprintEditorUtils::RemoveNode(NULL, SoundGraphNode, true);

				// Make sure SoundCue is updated to match graph
				SoundCue->CompileSoundNodesFromGraphNodes();

				// Remove this node from the SoundCue's list of all SoundNodes
				SoundCue->AllNodes.Remove(DelNode);
				SoundCue->MarkPackageDirty();
			}
			else
			{
				FBlueprintEditorUtils::RemoveNode(NULL, Node, true);
			}
		}
	}
}

bool FSoundCueEditor::CanDeleteNodes() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			if (Cast<USoundCueGraphNode_Root>(*NodeIt))
			{
				// Return false if only root node is selected, as it can't be deleted
				return false;
			}
		}
	}

	return SelectedNodes.Num() > 0;
}

void FSoundCueEditor::DeleteSelectedDuplicatableNodes()
{
	// Cache off the old selection
	const FGraphPanelSelectionSet OldSelectedNodes = GetSelectedNodes();

	// Clear the selection and only select the nodes that can be duplicated
	FGraphPanelSelectionSet RemainingNodes;
	SoundCueGraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			SoundCueGraphEditor->SetNodeSelection(Node, true);
		}
		else
		{
			RemainingNodes.Add(Node);
		}
	}

	// Delete the duplicatable nodes
	DeleteSelectedNodes();

	// Reselect whatever's left from the original selection after the deletion
	SoundCueGraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(RemainingNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			SoundCueGraphEditor->SetNodeSelection(Node, true);
		}
	}
}

void FSoundCueEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedDuplicatableNodes();
}

bool FSoundCueEditor::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FSoundCueEditor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	FString ExportedText;

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if(USoundCueGraphNode* Node = Cast<USoundCueGraphNode>(*SelectedIter))
		{
			Node->PrepareForCopying();
		}
	}

	FEdGraphUtilities::ExportNodesToText(SelectedNodes, /*out*/ ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	// Make sure SoundCue remains the owner of the copied nodes
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if (USoundCueGraphNode* Node = Cast<USoundCueGraphNode>(*SelectedIter))
		{
			Node->PostCopyNode();
		}
	}
}

bool FSoundCueEditor::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void FSoundCueEditor::PasteNodes()
{
	PasteNodesHere(SoundCueGraphEditor->GetPasteLocation2f());
}

void FSoundCueEditor::PasteNodesHere(const FVector2D& Location)
{
	// Undo/Redo support
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SoundCueEditorPaste", "Paste Sound Cue Node") );
	SoundCue->GetGraph()->Modify();
	SoundCue->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	SoundCueGraphEditor->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(SoundCue->GetGraph(), TextToImport, /*out*/ PastedNodes);

	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AvgNodePosition(0.0f,0.0f);

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;
		AvgNodePosition.X += Node->NodePosX;
		AvgNodePosition.Y += Node->NodePosY;
	}

	if ( PastedNodes.Num() > 0 )
	{
		float InvNumNodes = 1.0f/float(PastedNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;

		if (USoundCueGraphNode* SoundGraphNode = Cast<USoundCueGraphNode>(Node))
		{
			SoundCue->AllNodes.Add(SoundGraphNode->SoundNode);
		}

		// Select the newly pasted stuff
		SoundCueGraphEditor->SetNodeSelection(Node, true);

		Node->NodePosX = (Node->NodePosX - AvgNodePosition.X) + Location.X ;
		Node->NodePosY = (Node->NodePosY - AvgNodePosition.Y) + Location.Y ;

		Node->SnapToGrid(SNodePanel::GetSnapGridSize());

		// Give new node a different Guid from the old one
		Node->CreateNewGuid();
	}

	// Force new pasted SoundNodes to have same connections as graph nodes
	SoundCue->CompileSoundNodesFromGraphNodes();

	// Update UI
	SoundCueGraphEditor->NotifyGraphChanged();

	SoundCue->PostEditChange();
	SoundCue->MarkPackageDirty();
}

bool FSoundCueEditor::CanPasteNodes() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(SoundCue->SoundCueGraph, ClipboardContent);
}

void FSoundCueEditor::DuplicateNodes()
{
	// Copy and paste current selection
	CopySelectedNodes();
	PasteNodes();
}

bool FSoundCueEditor::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

void FSoundCueEditor::UndoGraphAction()
{
	GEditor->UndoTransaction();
}

void FSoundCueEditor::RedoGraphAction()
{
	// Clear selection, to avoid holding refs to nodes that go away
	SoundCueGraphEditor->ClearSelectionSet();

	GEditor->RedoTransaction();
}

void FSoundCueEditor::OnAlignTop()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnAlignTop();
	}
}

void FSoundCueEditor::OnAlignMiddle()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnAlignMiddle();
	}
}

void FSoundCueEditor::OnAlignBottom()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnAlignBottom();
	}
}

void FSoundCueEditor::OnAlignLeft()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnAlignLeft();
	}
}

void FSoundCueEditor::OnAlignCenter()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnAlignCenter();
	}
}

void FSoundCueEditor::OnAlignRight()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnAlignRight();
	}
}

void FSoundCueEditor::OnStraightenConnections()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnStraightenConnections();
	}
}

void FSoundCueEditor::OnDistributeNodesH()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnDistributeNodesH();
	}
}

void FSoundCueEditor::OnDistributeNodesV()
{
	if (SoundCueGraphEditor.IsValid())
	{
		SoundCueGraphEditor->OnDistributeNodesV();
	}
}

#undef LOCTEXT_NAMESPACE
