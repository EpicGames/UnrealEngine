// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Toolkits/IToolkitHost.h"
#include "ISoundCueEditor.h"
#include "Misc/NotifyHook.h"
#include "EditorUndoClient.h"

class IDetailsView;
class SGraphEditor;
class SSoundCuePalette;
class UEdGraphNode;
class USoundCue;
struct FPropertyChangedEvent;
struct Rect;

namespace Audio
{
	class FAudioDebugger;
}

/*-----------------------------------------------------------------------------
   FSoundCueEditor
-----------------------------------------------------------------------------*/

class FSoundCueEditor : public ISoundCueEditor, public FGCObject, public FNotifyHook, public FEditorUndoClient
{
public:
	FSoundCueEditor();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Destructor */
	virtual ~FSoundCueEditor();

	/** Edits the specified SoundCue object */
	void InitSoundCueEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit);

	/** ISoundCueEditor interface */
	virtual USoundCue* GetSoundCue() const override;
	virtual void SetSelection(TArray<UObject*> SelectedObjects) override;
	virtual bool GetBoundsForSelectedNodes(class FSlateRect& Rect, float Padding) override;
	virtual int32 GetNumberOfSelectedNodes() const override;
	virtual TSet<UObject*> GetSelectedNodes() const override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("Engine/Audio/SoundCues/Editor"));
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSoundCueEditor");
	}

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	TSharedRef<SDockTab> SpawnTab_GraphCanvas(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Palette(const FSpawnTabArgs& Args);

protected:
	/** Called when the preview text changes */
	void OnPreviewTextChanged(const FString& Text);

	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);

	/**
	 * Called when a node's title is committed for a rename
	 *
	 * @param	NewText				New title text
	 * @param	CommitInfo			How text was committed
	 * @param	NodeBeingChanged	The node being changed
	 */
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Select every node in the graph */
	void SelectAllNodes();
	/** Whether we can select every node */
	bool CanSelectAllNodes() const;

	/** Delete the currently selected nodes */
	void DeleteSelectedNodes();
	/** Whether we are able to delete the currently selected nodes */
	bool CanDeleteNodes() const;
	/** Delete only the currently selected nodes that can be duplicated */
	void DeleteSelectedDuplicatableNodes();

	/** Cut the currently selected nodes */
	void CutSelectedNodes();
	/** Whether we are able to cut the currently selected nodes */
	bool CanCutNodes() const;

	/** Copy the currently selected nodes */
	void CopySelectedNodes();
	/** Whether we are able to copy the currently selected nodes */
	bool CanCopyNodes() const;

	/** Paste the contents of the clipboard */
	void PasteNodes();
	/** Paste the contents of the clipboard at a specific location */
	virtual void PasteNodesHere(const FVector2D& Location) override;
	/** Whether we are able to paste the contents of the clipboard */
	virtual bool CanPasteNodes() const override;

	/** Duplicate the currently selected nodes */
	void DuplicateNodes();
	/** Whether we are able to duplicate the currently selected nodes */
	bool CanDuplicateNodes() const;

	/** Called to undo the last action */
	void UndoGraphAction();

	/** Called to redo the last undone action */
	void RedoGraphAction();

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();

	void OnStraightenConnections();

	void OnDistributeNodesH();
	void OnDistributeNodesV();

private:
	/** FNotifyHook interface */
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Builds the toolbar widget for the SoundCue editor */
	void ExtendToolbar();

	/** Binds new graph commands to delegates */
	void BindGraphCommands();

	/** Toolbar command methods */
	void PlayCue();
	void PlayNode();
	/** Whether we can play the current selection of nodes */
	bool CanPlayNode() const;
	void Stop();
	/** Either play the cue or stop currently playing sound */
	void TogglePlayback();

	/** Toggle solo explain soloing*/
	void ToggleSolo();
	bool CanExcuteToggleSolo() const;
	bool IsSoloToggled() const;
	
	/** Toggle mute */
	void ToggleMute();
	bool CanExcuteToggleMute() const;
	bool IsMuteToggled() const;
	
	/** Plays a single specified node */
	void PlaySingleNode(UEdGraphNode* Node);

	/** Sync the content browser to the current selection of nodes */
	void SyncInBrowser();
	/** Whether we can sync the content browser to the current selection of nodes */
	bool CanSyncInBrowser() const;

	/** Add an input to the currently selected node */
	void AddInput();
	/** Whether we can add an input to the currently selected node */
	bool CanAddInput() const;

	/** Delete an input from the currently selected node */
	void DeleteInput();
	/** Whether we can delete an input from the currently selected node */
	bool CanDeleteInput() const;

	/* Create comment node on graph */
	void OnCreateComment();

	/** Create new graph editor widget */
	TSharedRef<SGraphEditor> CreateGraphEditorWidget();

private:
	/** The SoundCue asset being inspected */
	TObjectPtr<USoundCue> SoundCue;

	/** New Graph Editor */
	TSharedPtr<SGraphEditor> SoundCueGraphEditor;

	/** Properties tab */
	TSharedPtr<class IDetailsView> SoundCueProperties;

	/** Palette of Sound Node types */
	TSharedPtr<class SSoundCuePalette> Palette;

	/** Command list for this editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

#if ENABLE_AUDIO_DEBUG
	/** Cache of the Audio debugger instance */
	Audio::FAudioDebugger* Debugger;
#endif

	/**	The tab ids for all the tabs used */
	static const FName GraphCanvasTabId;
	static const FName PropertiesTabId;
	static const FName PaletteTabId;
};
