// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueEditorUtilities.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/PlatformCrt.h"
#include "ISoundCueEditor.h"
#include "Misc/AssertionMacros.h"
#include "Sound/DialogueTypes.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "Templates/Casts.h"
#include "Toolkits/ToolkitManager.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

class IToolkit;

bool FSoundCueEditorUtilities::CanPasteNodes(const class UEdGraph* Graph)
{
	bool bCanPaste = false;
	TSharedPtr<class ISoundCueEditor> SoundCueEditor = GetISoundCueEditorForObject(Graph);
	if(SoundCueEditor.IsValid())
	{
		bCanPaste = SoundCueEditor->CanPasteNodes();
	}
	return bCanPaste;
}

void FSoundCueEditorUtilities::PasteNodesHere(class UEdGraph* Graph, const FVector2D& Location)
{
	TSharedPtr<class ISoundCueEditor> SoundCueEditor = GetISoundCueEditorForObject(Graph);
	if(SoundCueEditor.IsValid())
	{
		SoundCueEditor->PasteNodesHere(Location);
	}
}

void FSoundCueEditorUtilities::CreateWaveContainers(TArray<USoundWave*>& SelectedWaves, USoundCue* SoundCue, TArray<USoundNode*>& OutPlayers, FVector2D Location)
{
	const int32 NodeSpacing = 70;
	
	Location.Y -= static_cast<float>((SelectedWaves.Num() - 1) * NodeSpacing) / 2.f;

	for(int32 WaveIndex = 0; WaveIndex < SelectedWaves.Num(); WaveIndex++)
	{
		USoundWave* NewWave = SelectedWaves[WaveIndex];
		if(NewWave)
		{
			USoundNodeWavePlayer* WavePlayer = SoundCue->ConstructSoundNode<USoundNodeWavePlayer>();

			WavePlayer->SetSoundWave(NewWave);

			WavePlayer->GraphNode->NodePosX = Location.X - CastChecked<USoundCueGraphNode>(WavePlayer->GetGraphNode())->EstimateNodeWidth();
			WavePlayer->GraphNode->NodePosY = Location.Y + (NodeSpacing * WaveIndex);

			OutPlayers.Add(WavePlayer);
		}
	}
}

void FSoundCueEditorUtilities::CreateDialogueContainers(TArray<UDialogueWave*>& SelectedDialogues, USoundCue* SoundCue, TArray<USoundNode*>& OutPlayers, FVector2D Location)
{
	const int32 NodeSpacing = 70;

	Location.Y -= static_cast<float>((SelectedDialogues.Num() - 1) * NodeSpacing) / 2.f;

	for (int32 Index = 0; Index < SelectedDialogues.Num(); Index++)
	{
		UDialogueWave* NewDialogue = SelectedDialogues[Index];
		if (NewDialogue)
		{
			USoundNodeDialoguePlayer* DialoguePlayer = SoundCue->ConstructSoundNode<USoundNodeDialoguePlayer>();

			DialoguePlayer->SetDialogueWave(NewDialogue);

			if (NewDialogue->ContextMappings.Num() == 1)
			{
				DialoguePlayer->DialogueWaveParameter.Context.Speaker = NewDialogue->ContextMappings[0].Context.Speaker;
				DialoguePlayer->DialogueWaveParameter.Context.Targets = NewDialogue->ContextMappings[0].Context.Targets;
			}

			DialoguePlayer->GraphNode->NodePosX = Location.X - CastChecked<USoundCueGraphNode>(DialoguePlayer->GetGraphNode())->EstimateNodeWidth();
			DialoguePlayer->GraphNode->NodePosY = Location.Y + (NodeSpacing * Index);

			OutPlayers.Add(DialoguePlayer);
		}
	}
}

bool FSoundCueEditorUtilities::GetBoundsForSelectedNodes(const UEdGraph* Graph, class FSlateRect& Rect, float Padding)
{
	TSharedPtr<class ISoundCueEditor> SoundCueEditor = GetISoundCueEditorForObject(Graph);
	if(SoundCueEditor.IsValid())
	{
		return SoundCueEditor->GetBoundsForSelectedNodes(Rect, Padding);
	}
	return false;
}

int32 FSoundCueEditorUtilities::GetNumberOfSelectedNodes(const UEdGraph* Graph)
{
	TSharedPtr<class ISoundCueEditor> SoundCueEditor = GetISoundCueEditorForObject(Graph);
	if(SoundCueEditor.IsValid())
	{
		return SoundCueEditor->GetNumberOfSelectedNodes();
	}
	return 0;
}

TSet<UObject*> FSoundCueEditorUtilities::GetSelectedNodes(const UEdGraph* Graph)
{
	TSharedPtr<class ISoundCueEditor> SoundCueEditor = GetISoundCueEditorForObject(Graph);
	if(SoundCueEditor.IsValid())
	{
		return SoundCueEditor->GetSelectedNodes();
	}
	return TSet<UObject*>();
}

TSharedPtr<class ISoundCueEditor> FSoundCueEditorUtilities::GetISoundCueEditorForObject(const UObject* ObjectToFocusOn)
{
	check(ObjectToFocusOn);

	// Find the associated SoundCue
	USoundCue* SoundCue = Cast<const USoundCueGraph>(ObjectToFocusOn)->GetSoundCue();
	
	TSharedPtr<ISoundCueEditor> SoundCueEditor;
	if (SoundCue != NULL)
	{
		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(SoundCue);
		if (FoundAssetEditor.IsValid())
		{
			SoundCueEditor = StaticCastSharedPtr<ISoundCueEditor>(FoundAssetEditor);
		}
	}
	return SoundCueEditor;
}
