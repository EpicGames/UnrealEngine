// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraph/SoundCueGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "SoundCueGraph/SoundCueGraphNode_Root.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueGraph)

class FSoundCueAudioEditor : public ISoundCueAudioEditor
{
public:
	FSoundCueAudioEditor()
	{

	}

	~FSoundCueAudioEditor()
	{
	}

	UEdGraph* CreateNewSoundCueGraph(USoundCue* InSoundCue) override
	{
		USoundCueGraph* SoundCueGraph = CastChecked<USoundCueGraph>(FBlueprintEditorUtils::CreateNewGraph(InSoundCue, NAME_None, USoundCueGraph::StaticClass(), USoundCueGraphSchema::StaticClass()));

		return SoundCueGraph;
	}

	void SetupSoundNode(UEdGraph* SoundCueGraph, USoundNode* InSoundNode, bool bSelectNewNode) override
	{
		FGraphNodeCreator<USoundCueGraphNode> NodeCreator(*SoundCueGraph);
		USoundCueGraphNode* GraphNode = NodeCreator.CreateNode(bSelectNewNode);
		GraphNode->SetSoundNode(InSoundNode);
		NodeCreator.Finalize();
	}

	void LinkGraphNodesFromSoundNodes(USoundCue* SoundCue) override
	{
		// Use SoundNodes to make GraphNode Connections
		if (SoundCue->FirstNode != NULL)
		{
			// Find the root node
			TArray<USoundCueGraphNode_Root*> RootNodeList;
			SoundCue->SoundCueGraph->GetNodesOfClass<USoundCueGraphNode_Root>(/*out*/ RootNodeList);
			check(RootNodeList.Num() == 1);

			RootNodeList[0]->Pins[0]->BreakAllPinLinks();
			RootNodeList[0]->Pins[0]->MakeLinkTo(CastChecked<USoundCueGraphNode>(SoundCue->FirstNode->GetGraphNode())->GetOutputPin());
		}

		for (TArray<USoundNode*>::TConstIterator It(SoundCue->AllNodes); It; ++It)
		{
			USoundNode* SoundNode = *It;
			if (SoundNode)
			{
				TArray<UEdGraphPin*> InputPins;
				CastChecked<USoundCueGraphNode>(SoundNode->GetGraphNode())->GetInputPins(/*out*/ InputPins);
				check(InputPins.Num() == SoundNode->ChildNodes.Num());
				for (int32 ChildIndex = 0; ChildIndex < SoundNode->ChildNodes.Num(); ChildIndex++)
				{
					USoundNode* ChildNode = SoundNode->ChildNodes[ChildIndex];
					if (ChildNode)
					{
						InputPins[ChildIndex]->BreakAllPinLinks();
						InputPins[ChildIndex]->MakeLinkTo(CastChecked<USoundCueGraphNode>(ChildNode->GetGraphNode())->GetOutputPin());
					}
				}
			}
		}

	}

	void CompileSoundNodesFromGraphNodes(USoundCue* SoundCue) override
	{
		// Use GraphNodes to make SoundNode Connections
		TArray<USoundNode*> ChildNodes;
		TArray<UEdGraphPin*> InputPins;

		for (int32 NodeIndex = 0; NodeIndex < SoundCue->SoundCueGraph->Nodes.Num(); ++NodeIndex)
		{
			USoundCueGraphNode* GraphNode = Cast<USoundCueGraphNode>(SoundCue->SoundCueGraph->Nodes[NodeIndex]);
			if (GraphNode && GraphNode->SoundNode)
			{
				// Set ChildNodes of each SoundNode

				GraphNode->GetInputPins(InputPins);
				ChildNodes.Empty();
				for (int32 PinIndex = 0; PinIndex < InputPins.Num(); ++PinIndex)
				{
					UEdGraphPin* ChildPin = InputPins[PinIndex];

					if (ChildPin->LinkedTo.Num() > 0)
					{
						USoundCueGraphNode* GraphChildNode = CastChecked<USoundCueGraphNode>(ChildPin->LinkedTo[0]->GetOwningNode());
						ChildNodes.Add(GraphChildNode->SoundNode);
					}
					else
					{
						ChildNodes.AddZeroed();
					}
				}

				GraphNode->SoundNode->SetFlags(RF_Transactional);
				GraphNode->SoundNode->Modify();
				GraphNode->SoundNode->SetChildNodes(ChildNodes);
				GraphNode->SoundNode->PostEditChange();
			}
			else
			{
				// Set FirstNode based on RootNode connection
				USoundCueGraphNode_Root* RootNode = Cast<USoundCueGraphNode_Root>(SoundCue->SoundCueGraph->Nodes[NodeIndex]);
				if (RootNode)
				{
					SoundCue->Modify();
					if (RootNode->Pins[0]->LinkedTo.Num() > 0)
					{
						SoundCue->FirstNode = CastChecked<USoundCueGraphNode>(RootNode->Pins[0]->LinkedTo[0]->GetOwningNode())->SoundNode;
					}
					else
					{
						SoundCue->FirstNode = nullptr;
					}
					SoundCue->PostEditChange();
				}
			}
		}

	}

	void RemoveNullNodes(USoundCue* SoundCue) override
	{
		// Deal with SoundNode types being removed - iterate in reverse as nodes may be removed
		for (int32 idx = SoundCue->SoundCueGraph->Nodes.Num() - 1; idx >= 0; --idx)
		{
			USoundCueGraphNode* Node = Cast<USoundCueGraphNode>(SoundCue->SoundCueGraph->Nodes[idx]);

			if (Node && Node->SoundNode == NULL)
			{
				FBlueprintEditorUtils::RemoveNode(NULL, Node, true);
			}
		}
	}

	void CreateInputPin(UEdGraphNode* SoundCueNode) override
	{
		CastChecked<USoundCueGraphNode>(SoundCueNode)->CreateInputPin();
	}

	void RenameNodePins(USoundNode* SoundNode) override
	{
		TArray<class UEdGraphPin*> InputPins;

#if WITH_EDITORONLY_DATA
		CastChecked<USoundCueGraphNode>(SoundNode->GetGraphNode())->GetInputPins(InputPins);
#endif

		for (int32 i = 0; i < InputPins.Num(); i++)
		{
			if (InputPins[i])
			{
				InputPins[i]->PinName = *SoundNode->GetInputPinName(i).ToString();
			}
		}
	}
};


USoundCueGraph::USoundCueGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!USoundCue::GetSoundCueAudioEditor().IsValid())
	{
		USoundCue::SetSoundCueAudioEditor(TSharedPtr<ISoundCueAudioEditor>(new FSoundCueAudioEditor()));
	}
}

USoundCue* USoundCueGraph::GetSoundCue() const
{
	return CastChecked<USoundCue>(GetOuter());
}
