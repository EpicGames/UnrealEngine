// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueGraphConnectionDrawingPolicy.h"

#include "ActiveSound.h"
#include "Audio.h"
#include "AudioDevice.h"
#include "Components/AudioComponent.h"
#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Math/Vector2D.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraph/SoundCueGraphNode_Root.h"
#include "SoundCueGraph/SoundCueGraphSchema.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

class FArrangedChildren;
class FArrangedWidget;
class FSlateRect;
class SWidget;
class UEdGraphNode;

FConnectionDrawingPolicy* FSoundCueGraphConnectionDrawingPolicyFactory::CreateConnectionPolicy(const class UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const class FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	if (Schema->IsA(USoundCueGraphSchema::StaticClass()))
	{
		return new FSoundCueGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
	}
	return nullptr;
}


/////////////////////////////////////////////////////
// FSoundCueGraphConnectionDrawingPolicy

FSoundCueGraphConnectionDrawingPolicy::FSoundCueGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(InGraphObj)
{
	// Cache off the editor options
	ActiveColor = Settings->TraceAttackColor;
	InactiveColor = Settings->TraceReleaseColor;

	ActiveWireThickness = Settings->TraceAttackWireThickness;
	InactiveWireThickness = Settings->TraceReleaseWireThickness;

	// Don't want to draw ending arrowheads
	ArrowImage = nullptr;
	ArrowRadius = FVector2D::ZeroVector;
}

void FSoundCueGraphConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	// Build the execution roadmap (also populates execution times)
	BuildAudioFlowRoadmap();

	// Draw everything
	FConnectionDrawingPolicy::Draw(InPinGeometries, ArrangedNodes);
}

void FSoundCueGraphConnectionDrawingPolicy::BuildAudioFlowRoadmap()
{
	UAudioComponent* PreviewAudioComponent = GEditor->GetPreviewAudioComponent();
	FAudioDevice* AudioDevice = PreviewAudioComponent ? PreviewAudioComponent->GetAudioDevice() : nullptr;

	if (AudioDevice)
	{
		USoundCueGraph* SoundCueGraph = CastChecked<USoundCueGraph>(GraphObj);
		USoundCue* SoundCue = SoundCueGraph->GetSoundCue();


		if (PreviewAudioComponent && PreviewAudioComponent->IsPlaying() && PreviewAudioComponent->Sound == SoundCue)
		{
			TArray<FWaveInstance*> WaveInstances;
			const int32 FirstActiveIndex = AudioDevice->GetSortedActiveWaveInstances(WaveInstances, ESortedActiveWaveGetType::QueryOnly);

			// Run through the active instances and cull out anything that isn't related to this graph
			if (FirstActiveIndex > 0)
			{
				WaveInstances.RemoveAt(0, FirstActiveIndex + 1);
			}

			for (int32 WaveIndex = WaveInstances.Num() - 1; WaveIndex >= 0 ; --WaveIndex)
			{
				UAudioComponent* WaveInstanceAudioComponent = UAudioComponent::GetAudioComponentFromID(WaveInstances[WaveIndex]->ActiveSound->GetAudioComponentID());
				if (WaveInstanceAudioComponent != PreviewAudioComponent)
				{
					WaveInstances.RemoveAtSwap(WaveIndex);
				}
			}

			for (int32 WaveIndex = 0; WaveIndex < WaveInstances.Num(); ++WaveIndex)
			{
				TArray<USoundNode*> PathToWaveInstance;
				if (SoundCue->FindPathToNode(WaveInstances[WaveIndex]->WaveInstanceHash, PathToWaveInstance))
				{
					TArray<USoundCueGraphNode_Root*> RootNode;
					TArray<UEdGraphNode*> GraphNodes;
					SoundCueGraph->GetNodesOfClass<USoundCueGraphNode_Root>(RootNode);
					check(RootNode.Num() == 1);
					GraphNodes.Add(RootNode[0]);

					TArray<double> NodeTimes;
					NodeTimes.Add(FApp::GetCurrentTime()); // Time for the root node

					for (int32 i = 0; i < PathToWaveInstance.Num(); ++i)
					{
						const double ObservationTime = FApp::GetCurrentTime() + 1.f;

						NodeTimes.Add(ObservationTime);
						GraphNodes.Add(PathToWaveInstance[i]->GraphNode);
					}

					// Record the unique node->node pairings, keeping only the most recent times for each pairing
					for (int32 i = GraphNodes.Num() - 1; i >= 1; --i)
					{
						UEdGraphNode* CurNode = GraphNodes[i];
						double CurNodeTime = NodeTimes[i];
						UEdGraphNode* NextNode = GraphNodes[i-1];
						double NextNodeTime = NodeTimes[i-1];

						FExecPairingMap& Predecessors = PredecessorNodes.FindOrAdd(NextNode);

						// Update the timings if this is a more recent pairing
						FTimePair& Timings = Predecessors.FindOrAdd(CurNode);
						if (Timings.ThisExecTime < NextNodeTime)
						{
							Timings.PredExecTime = CurNodeTime;
							Timings.ThisExecTime = NextNodeTime;
						}
					}
				}
			}
		}
	}
}

// Give specific editor modes a chance to highlight this connection or darken non-interesting connections
void FSoundCueGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;

	// Get the schema and grab the default color from it
	check(OutputPin);
	check(GraphObj);
	const UEdGraphSchema* Schema = GraphObj->GetSchema();

	Params.WireColor = Schema->GetPinTypeColor(OutputPin->PinType);

	if (InputPin == NULL)
	{
		return;
	}
	
	bool bExecuted = false;

	// Run thru the predecessors, and on
	if (FExecPairingMap* PredecessorMap = PredecessorNodes.Find(InputPin->GetOwningNode()))
	{
		if (FTimePair* Times = PredecessorMap->Find(OutputPin->GetOwningNode()))
		{
			bExecuted = true;

			Params.WireThickness = ActiveWireThickness;
			Params.WireColor = ActiveColor;

			Params.bDrawBubbles = true;
		}
	}

	if (!bExecuted)
	{
		// It's not followed, fade it and keep it thin
		Params.WireColor = InactiveColor;
		Params.WireThickness = InactiveWireThickness;
	}
}
