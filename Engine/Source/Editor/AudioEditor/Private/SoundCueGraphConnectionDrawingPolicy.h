// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

class FArrangedChildren;
class FArrangedWidget;
class FSlateRect;
class FSlateWindowElementList;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;

struct FSoundCueGraphConnectionDrawingPolicyFactory : public FGraphPanelPinConnectionFactory
{
public:
	virtual ~FSoundCueGraphConnectionDrawingPolicyFactory() = default;

	// FGraphPanelPinConnectionFactory
	virtual class FConnectionDrawingPolicy* CreateConnectionPolicy(const class UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const class FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	// ~FGraphPanelPinConnectionFactory

};


/////////////////////////////////////////////////////
// FSoundCueGraphConnectionDrawingPolicy

// This class draws the connections for an UEdGraph using a SoundCue schema
class FSoundCueGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	// Times for one execution pair within the current graph
	struct FTimePair
	{
		double PredExecTime;
		double ThisExecTime;

		FTimePair()
			: PredExecTime(0.0)
			, ThisExecTime(0.0)
		{
		}
	};

	// Map of pairings
	typedef TMap<UEdGraphNode*, FTimePair> FExecPairingMap;

	// Map of nodes that preceeded before a given node in the execution sequence (one entry for each pairing)
	TMap<UEdGraphNode*, FExecPairingMap> PredecessorNodes;

	UEdGraph* GraphObj;

	FLinearColor ActiveColor;
	FLinearColor InactiveColor;

	float ActiveWireThickness;
	float InactiveWireThickness;

public:
	FSoundCueGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	void BuildAudioFlowRoadmap();

	// FConnectionDrawingPolicy interface
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& PinGeometries, FArrangedChildren& ArrangedNodes) override;
	// End of FConnectionDrawingPolicy interface
};
