// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueGraphEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "SoundCueGraphEditorCommands"

void FSoundCueGraphEditorCommands::RegisterCommands()
{
	UI_COMMAND(PlayCue, "Play Cue", "Plays the SoundCue", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayNode, "Play Node", "Plays the currently selected node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopCueNode, "Stop", "Stops the currently playing cue/node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TogglePlayback, "Toggle Playback", "Plays the SoundCue or stops the currently playing cue/node", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));

	UI_COMMAND(BrowserSync, "Sync to Browser", "Selects the SoundWave in the content browser", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddInput, "Add Input", "Adds an input to the node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteInput, "Delete Input", "Removes an input from the node", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleSolo, "Solo", "Toggles Soloing this cue", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::S));
	UI_COMMAND(ToggleMute, "Mute", "Toggles Muting this cue", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::M));
}

#undef LOCTEXT_NAMESPACE
