// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueGraph/SoundCueGraphNode.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "Editor/EditorEngine.h"
#include "Engine/Font.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "ScopedTransaction.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraphEditorCommands.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueGraphNode)

#define LOCTEXT_NAMESPACE "SoundCueGraphNode"

/////////////////////////////////////////////////////
// USoundCueGraphNode

USoundCueGraphNode::USoundCueGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundCueGraphNode::PostLoad()
{
	Super::PostLoad();

	// Fixup any SoundNode back pointers that may be out of date
	if (SoundNode)
	{
		SoundNode->GraphNode = this;
	}

	for (int32 Index = 0; Index < Pins.Num(); ++Index)
	{
		UEdGraphPin* Pin = Pins[Index];
		if (Pin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			if (Pin->Direction == EGPD_Input)
			{
				Pin->PinName = CreateUniquePinName(TEXT("Input"));
			}
			else
			{
				Pin->PinName = CreateUniquePinName(TEXT("Output"));
			}
			Pin->PinFriendlyName = FText::FromString(TEXT(" "));
		}
	}
}

void USoundCueGraphNode::SetSoundNode(USoundNode* InSoundNode)
{
	SoundNode = InSoundNode;
	InSoundNode->GraphNode = this;
}

void USoundCueGraphNode::CreateInputPin()
{
	UEdGraphPin* NewPin = CreatePin(EGPD_Input, TEXT("SoundNode"), *SoundNode->GetInputPinName(GetInputCount()).ToString());
	if (NewPin->PinName.IsNone())
	{
		// Makes sure pin has a name for lookup purposes but user will never see it
		NewPin->PinName = CreateUniquePinName(TEXT("Input"));
		NewPin->PinFriendlyName = FText::FromString(TEXT(" "));
	}
}

void USoundCueGraphNode::AddInputPin()
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SoundCueEditorAddInput", "Add Sound Cue Input") );
	Modify();
	CreateInputPin();

	USoundCue* SoundCue = CastChecked<USoundCueGraph>(GetGraph())->GetSoundCue();
	SoundCue->CompileSoundNodesFromGraphNodes();
	SoundCue->MarkPackageDirty();

	// Refresh the current graph, so the pins can be updated
	GetGraph()->NotifyGraphChanged();
}

void USoundCueGraphNode::RemoveInputPin(UEdGraphPin* InGraphPin)
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SoundCueEditorDeleteInput", "Delete Sound Cue Input") );
	Modify();

	TArray<class UEdGraphPin*> InputPins;
	GetInputPins(InputPins);

	for (int32 InputIndex = 0; InputIndex < InputPins.Num(); InputIndex++)
	{
		if (InGraphPin == InputPins[InputIndex])
		{
			InGraphPin->MarkAsGarbage();
			Pins.Remove(InGraphPin);
			// also remove the SoundNode child node so ordering matches
			SoundNode->Modify();
			SoundNode->RemoveChildNode(InputIndex);
			break;
		}
	}

	USoundCue* SoundCue = CastChecked<USoundCueGraph>(GetGraph())->GetSoundCue();
	SoundCue->CompileSoundNodesFromGraphNodes();
	SoundCue->MarkPackageDirty();

	// Refresh the current graph, so the pins can be updated
	GetGraph()->NotifyGraphChanged();
}

int32 USoundCueGraphNode::EstimateNodeWidth() const
{
	const int32 EstimatedCharWidth = 6;
	FString NodeTitle = GetNodeTitle(ENodeTitleType::FullTitle).ToString();
	UFont* Font = GetDefault<UEditorEngine>()->EditorFont;
	int32 Result = NodeTitle.Len()*EstimatedCharWidth;

	if (Font)
	{
		Result = Font->GetStringSize(*NodeTitle);
	}

	return Result;
}

bool USoundCueGraphNode::CanAddInputPin() const
{
	if(SoundNode)
	{
		// Check if adding another input would exceed max child nodes.
		return SoundNode->ChildNodes.Num() < SoundNode->GetMaxChildNodes();
	}
	else
	{
		return false;
	}
}

FText USoundCueGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (SoundNode)
	{
		return SoundNode->GetTitle();
	}
	else
	{
		return Super::GetNodeTitle(TitleType);
	}
}

void USoundCueGraphNode::PrepareForCopying()
{
	if (SoundNode)
	{
		// Temporarily take ownership of the SoundNode, so that it is not deleted when cutting
		SoundNode->Rename(NULL, this, REN_DontCreateRedirectors);
	}
}

void USoundCueGraphNode::PostCopyNode()
{
	// Make sure the SoundNode goes back to being owned by the SoundCue after copying.
	ResetSoundNodeOwner();
}

void USoundCueGraphNode::PostEditImport()
{
	// Make sure this SoundNode is owned by the SoundCue it's being pasted into.
	ResetSoundNodeOwner();
}

void USoundCueGraphNode::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		CreateNewGuid();
	}
}

void USoundCueGraphNode::ResetSoundNodeOwner()
{
	if (SoundNode)
	{
		USoundCue* SoundCue = CastChecked<USoundCueGraph>(GetGraph())->GetSoundCue();

		if (SoundNode->GetOuter() != SoundCue)
		{
			// Ensures SoundNode is owned by the SoundCue
			SoundNode->Rename(NULL, SoundCue, REN_DontCreateRedirectors);
		}

		// Set up the back pointer for newly created sound nodes
		SoundNode->GraphNode = this;
	}
}

void USoundCueGraphNode::CreateInputPins()
{
	if (SoundNode)
	{
		for (int32 ChildIndex = 0; ChildIndex < SoundNode->ChildNodes.Num(); ++ChildIndex)
		{
			CreateInputPin();
		}
	}
}

void USoundCueGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Pin)
	{
		// If on an input that can be deleted, show option
		if(Context->Pin->Direction == EGPD_Input && SoundNode->ChildNodes.Num() > SoundNode->GetMinChildNodes())
		{
			FToolMenuSection& Section = Menu->AddSection("SoundCueGraphDeleteInput");
			Section.AddMenuEntry(FSoundCueGraphEditorCommands::Get().DeleteInput);
		}
	}
	else if (Context->Node)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("SoundCueGraphNodeAlignment");
			Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* SubMenu)
			{
				{
					FToolMenuSection& SubMenuSection = SubMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
				}

				{
					FToolMenuSection& SubMenuSection = SubMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
					SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
				}
			}));
		}

		{
			FToolMenuSection& Section = Menu->AddSection("SoundCueGraphNodeEdit");
			Section.AddMenuEntry( FGenericCommands::Get().Delete );
			Section.AddMenuEntry( FGenericCommands::Get().Cut );
			Section.AddMenuEntry( FGenericCommands::Get().Copy );
			Section.AddMenuEntry( FGenericCommands::Get().Duplicate );
		}

		{
			FToolMenuSection& Section = Menu->AddSection("SoundCueGraphNodeAddPlaySync");
			if (CanAddInputPin())
			{
				Section.AddMenuEntry(FSoundCueGraphEditorCommands::Get().AddInput);
			}

			Section.AddMenuEntry(FSoundCueGraphEditorCommands::Get().PlayNode);

			if ( Cast<USoundNodeWavePlayer>(SoundNode) )
			{
				Section.AddMenuEntry(FSoundCueGraphEditorCommands::Get().BrowserSync);
			}
			else if (Cast<USoundNodeDialoguePlayer>(SoundNode))
			{
				Section.AddMenuEntry(FSoundCueGraphEditorCommands::Get().BrowserSync);
			}
		}
	}
}

FText USoundCueGraphNode::GetTooltipText() const
{
	FText Tooltip;
	if (SoundNode)
	{
		Tooltip = SoundNode->GetClass()->GetToolTipText();
	}
	if (Tooltip.IsEmpty())
	{
		Tooltip = GetNodeTitle(ENodeTitleType::ListView);
	}
	return Tooltip;
}

FString USoundCueGraphNode::GetDocumentationExcerptName() const
{
	// Default the node to searching for an excerpt named for the C++ node class name, including the U prefix.
	// This is done so that the excerpt name in the doc file can be found by find-in-files when searching for the full class name.
	UClass* MyClass = (SoundNode != NULL) ? SoundNode->GetClass() : this->GetClass();
	return FString::Printf(TEXT("%s%s"), MyClass->GetPrefixCPP(), *MyClass->GetName());
}

#undef LOCTEXT_NAMESPACE
