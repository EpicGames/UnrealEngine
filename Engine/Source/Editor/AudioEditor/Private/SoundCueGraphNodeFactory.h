// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"
#include "SoundCueGraph/SoundCueGraphNode_Base.h"
#include "SoundCueGraph/SoundCueGraphNode_Root.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphNode.h"
#include "SGraphNodeSoundResult.h"
#include "SGraphNodeSoundBase.h"


class FSoundCueGraphNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<class SGraphNode> CreateNode(class UEdGraphNode* InNode) const override
	{
		if (USoundCueGraphNode_Base* BaseSoundNode = Cast<USoundCueGraphNode_Base>(InNode))
		{
			if (USoundCueGraphNode_Root* RootSoundNode = Cast<USoundCueGraphNode_Root>(InNode))
			{
				return SNew(SGraphNodeSoundResult, RootSoundNode);
			}
			else if (USoundCueGraphNode* SoundNode = Cast<USoundCueGraphNode>(InNode))
			{
				return SNew(SGraphNodeSoundBase, SoundNode);
			}
		}
		return nullptr;
	}
};
