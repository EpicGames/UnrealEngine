// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SoundCueGraphNode_Base.cpp
=============================================================================*/

#include "SoundCueGraph/SoundCueGraphNode_Base.h"

#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "SoundCueGraph/SoundCueGraphSchema.h"
#include "Templates/Casts.h"

/////////////////////////////////////////////////////
// USoundCueGraphNode_Base

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueGraphNode_Base)

USoundCueGraphNode_Base::USoundCueGraphNode_Base(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UEdGraphPin* USoundCueGraphNode_Base::GetOutputPin()
{
	UEdGraphPin* OutputPin = NULL;

	for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Output)
		{
			// should only ever be one output pin
			check(OutputPin == NULL);
			OutputPin = Pins[PinIndex];
		}
	}

	return OutputPin;
}

void USoundCueGraphNode_Base::GetInputPins(TArray<class UEdGraphPin*>& OutInputPins)
{
	OutInputPins.Empty();

	for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			OutInputPins.Add(Pins[PinIndex]);
		}
	}
}

UEdGraphPin* USoundCueGraphNode_Base::GetInputPin(int32 InputIndex)
{
	check(InputIndex >= 0 && InputIndex < GetInputCount());

	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			if (InputIndex == FoundInputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundInputs++;
			}
		}
	}

	return NULL;
}

int32 USoundCueGraphNode_Base::GetInputCount() const
{
	int32 InputCount = 0;

	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			InputCount++;
		}
	}

	return InputCount;
}

void USoundCueGraphNode_Base::InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList)
{
	const USoundCueGraphSchema* Schema = CastChecked<USoundCueGraphSchema>(GetSchema());

	// The pin we are creating from already has a connection that needs to be broken. We want to "insert" the new node in between, so that the output of the new node is hooked up too
	UEdGraphPin* OldLinkedPin = FromPin->LinkedTo[0];
	check(OldLinkedPin);

	FromPin->BreakAllPinLinks();

	// Hook up the old linked pin to the first valid output pin on the new node
	for (int32 OutpinPinIdx=0; OutpinPinIdx<Pins.Num(); OutpinPinIdx++)
	{
		UEdGraphPin* OutputExecPin = Pins[OutpinPinIdx];
		check(OutputExecPin);
		if (ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE == Schema->CanCreateConnection(OldLinkedPin, OutputExecPin).Response)
		{
			if (Schema->TryCreateConnection(OldLinkedPin, OutputExecPin))
			{
				OutNodeList.Add(OldLinkedPin->GetOwningNode());
				OutNodeList.Add(this);
			}
			break;
		}
	}

	if (Schema->TryCreateConnection(FromPin, NewLinkPin))
	{
		OutNodeList.Add(FromPin->GetOwningNode());
		OutNodeList.Add(this);
	}
}

void USoundCueGraphNode_Base::AllocateDefaultPins()
{
	check(Pins.Num() == 0);

	CreateInputPins();

	if (!IsRootNode())
	{
		CreatePin(EGPD_Output, TEXT("SoundNode"), TEXT("Output"));
	}
}

void USoundCueGraphNode_Base::ReconstructNode()
{
	// Break any links to 'orphan' pins
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		TArray<class UEdGraphPin*>& LinkedToRef = Pin->LinkedTo;
		for (int32 LinkIdx=0; LinkIdx < LinkedToRef.Num(); LinkIdx++)
		{
			UEdGraphPin* OtherPin = LinkedToRef[LinkIdx];
			// If we are linked to a pin that its owner doesn't know about, break that link
			if (!OtherPin->GetOwningNode()->Pins.Contains(OtherPin))
			{
				Pin->LinkedTo.Remove(OtherPin);
			}
		}
	}

	// Store the old Input and Output pins
	TArray<UEdGraphPin*> OldInputPins;
	GetInputPins(OldInputPins);
	UEdGraphPin* OldOutputPin = GetOutputPin();

	// Move the existing pins to a saved array
	TArray<UEdGraphPin*> OldPins(Pins);
	Pins.Reset();

	// Recreate the new pins
	AllocateDefaultPins();

	// Get new Input and Output pins
	TArray<UEdGraphPin*> NewInputPins;
	GetInputPins(NewInputPins);
	UEdGraphPin* NewOutputPin = GetOutputPin();

	for (int32 PinIndex = 0; PinIndex < OldInputPins.Num(); PinIndex++)
	{
		if (PinIndex < NewInputPins.Num())
		{
			NewInputPins[PinIndex]->MovePersistentDataFromOldPin(*OldInputPins[PinIndex]);
		}
	}

	if (OldOutputPin)
	{
		NewOutputPin->MovePersistentDataFromOldPin(*OldOutputPin);
	}

	// Throw away the original pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		OldPin->Modify();
		UEdGraphNode::DestroyPin(OldPin);
	}
}

void USoundCueGraphNode_Base::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin != NULL)
	{
		const USoundCueGraphSchema* Schema = CastChecked<USoundCueGraphSchema>(GetSchema());

		TSet<UEdGraphNode*> NodeList;

		// auto-connect from dragged pin to first compatible pin on the new node
		for (int32 i=0; i<Pins.Num(); i++)
		{
			UEdGraphPin* Pin = Pins[i];
			check(Pin);
			FPinConnectionResponse Response = Schema->CanCreateConnection(FromPin, Pin);
			if (ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE == Response.Response) //-V1051
			{
				if (Schema->TryCreateConnection(FromPin, Pin))
				{
					NodeList.Add(FromPin->GetOwningNode());
					NodeList.Add(this);
				}
				break;
			}
			else if(ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_A == Response.Response)
			{
				InsertNewNode(FromPin, Pin, NodeList);
				break;
			}
		}

		// Send all nodes that received a new pin connection a notification
		for (auto It = NodeList.CreateConstIterator(); It; ++It)
		{
			UEdGraphNode* Node = (*It);
			Node->NodeConnectionListChanged();
		}
	}
}

bool USoundCueGraphNode_Base::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(USoundCueGraphSchema::StaticClass());
}

FString USoundCueGraphNode_Base::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/SoundCue");
}
