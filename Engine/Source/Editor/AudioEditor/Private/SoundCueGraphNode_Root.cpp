// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SoundCueGraphNode_Root.cpp
=============================================================================*/

#include "SoundCueGraph/SoundCueGraphNode_Root.h"

#include "GraphEditorSettings.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "SoundCueGraphEditorCommands.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueGraphNode_Root)

#define LOCTEXT_NAMESPACE "SoundCueGraphNode_Root"

/////////////////////////////////////////////////////
// USoundCueGraphNode_Root

USoundCueGraphNode_Root::USoundCueGraphNode_Root(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor USoundCueGraphNode_Root::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText USoundCueGraphNode_Root::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("RootTitle", "Output");
}

FText USoundCueGraphNode_Root::GetTooltipText() const
{
	return LOCTEXT("RootToolTip", "Wire the final Sound Node into this node");
}

void USoundCueGraphNode_Root::CreateInputPins()
{
	CreatePin(EGPD_Input, TEXT("SoundNode"), TEXT("Root"), NAME_None);
}

void USoundCueGraphNode_Root::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Pin)
	{
	}
	else if (Context->Node)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("SoundCueGraphNodePlay");
			Section.AddMenuEntry(FSoundCueGraphEditorCommands::Get().PlayNode);
		}
	}
}

#undef LOCTEXT_NAMESPACE
