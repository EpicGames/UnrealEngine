// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SoundCueGraphSchema.cpp
=============================================================================*/

#include "SoundCueGraph/SoundCueGraphSchema.h"

#include "AssetRegistry/AssetData.h"
#include "ClassViewerFilter.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphNode_Comment.h"
#include "Editor.h"
#include "Editor/AssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "Internationalization/Internationalization.h"
#include "Layout/SlateRect.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Sound/DialogueTypes.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundWave.h"
#include "SoundCueEditorUtilities.h"
#include "SSoundCuePalette.h"
#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "SoundCueGraph/SoundCueGraphNode_Root.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueGraphSchema)

class FString;

#define LOCTEXT_NAMESPACE "SoundCueSchema"

/////////////////////////////////////////////////////
// FSoundCueGraphSchemaAction_NewNode

UEdGraphNode* FSoundCueGraphSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	check(SoundNodeClass);

	USoundCue* SoundCue = CastChecked<USoundCueGraph>(ParentGraph)->GetSoundCue();
	const FScopedTransaction Transaction( LOCTEXT("SoundCueEditorNewSoundNode", "Sound Cue Editor: New Sound Node") );
	ParentGraph->Modify();
	SoundCue->Modify();

	USoundNode* NewNode = SoundCue->ConstructSoundNode<USoundNode>(SoundNodeClass, bSelectNewNode);

	// If this node allows >0 children but by default has zero - create a connector for starters
	if (NewNode->GetMaxChildNodes() > 0 && NewNode->ChildNodes.Num() == 0)
	{
		NewNode->CreateStartingConnectors();
	}

	// Attempt to connect inputs to selected nodes, unless we're already dragging from a single output
	if (FromPin == NULL || FromPin->Direction == EGPD_Input)
	{
		ConnectToSelectedNodes(NewNode, ParentGraph);
	}

	NewNode->GraphNode->NodePosX = Location.X;
	NewNode->GraphNode->NodePosY = Location.Y;

	NewNode->GraphNode->AutowireNewNode(FromPin);

	SoundCue->PostEditChange();
	SoundCue->MarkPackageDirty();

	return NewNode->GraphNode;
}

void FSoundCueGraphSchemaAction_NewNode::ConnectToSelectedNodes(USoundNode* NewNode, class UEdGraph* ParentGraph) const
{
	// only connect if node can have many children
	if (NewNode->GetMaxChildNodes() > 1)
	{
		const FGraphPanelSelectionSet SelectedNodes = FSoundCueEditorUtilities::GetSelectedNodes(ParentGraph);

		TArray<USoundNode*> SortedNodes;
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			USoundCueGraphNode* SelectedNode = Cast<USoundCueGraphNode>(*NodeIt);

			if (SelectedNode)
			{
				// Sort the nodes by y position
				bool bInserted = false;
				for (int32 Index = 0; Index < SortedNodes.Num(); ++Index)
				{
					if (SortedNodes[Index]->GraphNode->NodePosY > SelectedNode->NodePosY)
					{
						SortedNodes.Insert(SelectedNode->SoundNode, Index);
						bInserted = true;
						break;
					}
				}
				if (!bInserted)
				{
					SortedNodes.Add(SelectedNode->SoundNode);
				}
			}
		}
		if (SortedNodes.Num() > 1)
		{
			CastChecked<USoundCueGraphSchema>(NewNode->GraphNode->GetSchema())->TryConnectNodes(SortedNodes, NewNode);
		}
	}
}

/////////////////////////////////////////////////////
// FSoundCueGraphSchemaAction_NewFromSelected

UEdGraphNode* FSoundCueGraphSchemaAction_NewFromSelected::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	USoundCue* SoundCue = CastChecked<USoundCueGraph>(ParentGraph)->GetSoundCue();
	const FScopedTransaction Transaction( LOCTEXT("SoundCueEditorNewFromSelection", "Sound Cue Editor: New From Selection") );
	ParentGraph->Modify();
	SoundCue->Modify();

	UEdGraphNode* CreatedNode = NULL;

	FDeprecateSlateVector2D WaveStartLocation = Location;

	if (SoundNodeClass)
	{
		// If we will create another node, move wave nodes out of the way.
		WaveStartLocation.X -= 200.0f;
	}

	TArray<USoundWave*> SelectedWaves;
	TArray<UDialogueWave*> SelectedDialogues;
	TArray<USoundNode*> CreatedPlayers;

	GEditor->GetSelectedObjects()->GetSelectedObjects<USoundWave>(SelectedWaves);
	GEditor->GetSelectedObjects()->GetSelectedObjects<UDialogueWave>(SelectedDialogues);

	FSoundCueEditorUtilities::CreateWaveContainers(SelectedWaves, SoundCue, CreatedPlayers, WaveStartLocation);
	FSoundCueEditorUtilities::CreateDialogueContainers(SelectedDialogues, SoundCue, CreatedPlayers, WaveStartLocation);

	if (SoundNodeClass)
	{
		USoundNode* NewNode = SoundCue->ConstructSoundNode<USoundNode>(SoundNodeClass, bSelectNewNode);
		UEdGraphNode* NewGraphNode = NewNode->GraphNode;
		const USoundCueGraphSchema* NewSchema = CastChecked<USoundCueGraphSchema>(NewGraphNode->GetSchema());

		// If this node allows >0 children but by default has zero - create a connector for starters
		if (NewNode->GetMaxChildNodes() > 0 && NewNode->ChildNodes.Num() == 0)
		{
			NewNode->CreateStartingConnectors();
		}

		NewSchema->TryConnectNodes(CreatedPlayers, NewNode);

		NewGraphNode->NodePosX = Location.X;
		NewGraphNode->NodePosY = Location.Y;

		CreatedNode = NewNode->GraphNode;
	}
	else
	{
		if (CreatedPlayers.Num() > 0)
		{
			CreatedNode = CreatedPlayers[0]->GraphNode;
		}
	}

	if (CreatedNode)
	{
		CreatedNode->AutowireNewNode(FromPin);
	}

	SoundCue->PostEditChange();
	SoundCue->MarkPackageDirty();

	return CreatedNode;
}

/////////////////////////////////////////////////////
// FSoundCueGraphSchemaAction_NewComment

UEdGraphNode* FSoundCueGraphSchemaAction_NewComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	// Add menu item for creating comment boxes
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	FVector2f SpawnLocation = Location;

	FSlateRect Bounds;
	if (FSoundCueEditorUtilities::GetBoundsForSelectedNodes(ParentGraph, Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}

	return FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation);
}

/////////////////////////////////////////////////////
// FSoundCueGraphSchemaAction_Paste

UEdGraphNode* FSoundCueGraphSchemaAction_Paste::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	FSoundCueEditorUtilities::PasteNodesHere(ParentGraph, FDeprecateSlateVector2D(Location));
	return NULL;
}

/////////////////////////////////////////////////////
// USoundCueGraphSchema

USoundCueGraphSchema::USoundCueGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundCueGraphSchema::UpdateSoundNodeList(const SSoundCuePalette::FSoundNodeFilterData& FilterData)
{
	check(FilterData.InitOptions.IsValid());
	check(FilterData.ClassFilter.IsValid());
	check(FilterData.FilterFuncs.IsValid());
	
	AllowedSoundNodes.Empty();
	TArray<UClass*> SoundNodeClasses;
	GetDerivedClasses(USoundNode::StaticClass(), SoundNodeClasses, true);
	SoundNodeClasses.Sort();
	
	for(TSubclassOf<USoundNode> SoundNodeClass : SoundNodeClasses)
	{		
		if(!SoundNodeClass->HasAnyClassFlags(CLASS_Abstract) &&
		FilterData.ClassFilter->IsClassAllowed(*FilterData.InitOptions, SoundNodeClass, FilterData.FilterFuncs.ToSharedRef()))
		{
			AllowedSoundNodes.Add(SoundNodeClass);
		}
	}
}

bool USoundCueGraphSchema::ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	USoundCueGraphNode* InputNode = Cast<USoundCueGraphNode>(InputPin->GetOwningNode());

	if (InputNode)
	{
		// Only nodes representing SoundNodes have outputs
		USoundCueGraphNode* OutputNode = CastChecked<USoundCueGraphNode>(OutputPin->GetOwningNode());

		if (OutputNode->SoundNode)
		{
			// Grab all child nodes. We can't just test the output because 
			// the loop could happen from any additional child nodes. 
			TArray<USoundNode*> Nodes;
			OutputNode->SoundNode->GetAllNodes(Nodes);

			// If our test input is in that set, return true.
			return Nodes.Contains(InputNode->SoundNode);
		}
	}

	// Simple connection to root node
	return false;
}

void USoundCueGraphSchema::GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder) const
{
	GetAllSoundNodeActions(ActionMenuBuilder, false);
	GetCommentAction(ActionMenuBuilder);
}

void USoundCueGraphSchema::TryConnectNodes(const TArray<USoundNode*>& OutputNodes, USoundNode* InputNode) const
{
	for (int32 Index = 0; Index < OutputNodes.Num(); Index++)
	{
		if ( Index < InputNode->GetMaxChildNodes() )
		{
			USoundCueGraphNode* GraphNode = CastChecked<USoundCueGraphNode>(InputNode->GetGraphNode());
			if (Index >= GraphNode->GetInputCount())
			{
				GraphNode->CreateInputPin();
			}
			TryCreateConnection(GraphNode->GetInputPin(Index), CastChecked<USoundCueGraphNode>(OutputNodes[Index]->GetGraphNode())->GetOutputPin() );
		}
	}
}

void USoundCueGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	GetAllSoundNodeActions(ContextMenuBuilder, true);

	GetCommentAction(ContextMenuBuilder, ContextMenuBuilder.CurrentGraph);

	if (!ContextMenuBuilder.FromPin && FSoundCueEditorUtilities::CanPasteNodes(ContextMenuBuilder.CurrentGraph))
	{
		TSharedPtr<FSoundCueGraphSchemaAction_Paste> NewAction( new FSoundCueGraphSchemaAction_Paste(FText::GetEmpty(), LOCTEXT("PasteHereAction", "Paste here"), FText::GetEmpty(), 0) );
		ContextMenuBuilder.AddAction( NewAction );
	}
}

void USoundCueGraphSchema::GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node)
	{
		const USoundCueGraphNode* SoundGraphNode = Cast<const USoundCueGraphNode>(Context->Node);
		{
			FToolMenuSection& Section = Menu->AddSection("SoundCueGraphSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
		}
	}

	Super::GetContextMenuActions(Menu, Context);
}

void USoundCueGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	const int32 RootNodeHeightOffset = -58;

	// Create the result node
	FGraphNodeCreator<USoundCueGraphNode_Root> NodeCreator(Graph);
	USoundCueGraphNode_Root* ResultRootNode = NodeCreator.CreateNode();
	ResultRootNode->NodePosY = RootNodeHeightOffset;
	NodeCreator.Finalize();
	SetNodeMetaData(ResultRootNode, FNodeMetadata::DefaultGraphNode);
}

const FPinConnectionResponse USoundCueGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = NULL;
	const UEdGraphPin* OutputPin = NULL;

	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatible", "Directions are not compatible"));
	}

	if (ConnectionCausesLoop(InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop"));
	}

	// Break existing connections on inputs only - multiple output connections are acceptable
	if (InputPin->LinkedTo.Num() > 0)
	{
		ECanCreateConnectionResponse ReplyBreakOutputs;
		if (InputPin == PinA)
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
		}
		else
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
		}
		return FPinConnectionResponse(ReplyBreakOutputs, LOCTEXT("ConnectionReplace", "Replace existing connections"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

bool USoundCueGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	bool bModified = UEdGraphSchema::TryCreateConnection(PinA, PinB);

	if (bModified)
	{
		CastChecked<USoundCueGraph>(PinA->GetOwningNode()->GetGraph())->GetSoundCue()->CompileSoundNodesFromGraphNodes();
	}

	return bModified;
}

bool USoundCueGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	return true;
}

FLinearColor USoundCueGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::White;
}

void USoundCueGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	Super::BreakNodeLinks(TargetNode);

	CastChecked<USoundCueGraph>(TargetNode.GetGraph())->GetSoundCue()->CompileSoundNodesFromGraphNodes();
}

void USoundCueGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links") );

	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);

	// if this would notify the node then we need to compile the SoundCue
	if (bSendsNodeNotifcation)
	{
		CastChecked<USoundCueGraph>(TargetPin.GetOwningNode()->GetGraph())->GetSoundCue()->CompileSoundNodesFromGraphNodes();
	}
}

void USoundCueGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
	for (const FAssetData& AssetData : Assets)
	{
		const UObject* Asset = AssetData.GetAsset();
		// As soon as one of the items is a sound or dialogue wave, say we can drag it on... we actually eat only the sound waves.
		if (Cast<const USoundWave>(Asset) != nullptr || Cast<const UDialogueWave>(Asset) != nullptr)
		{
			if (AssetReferenceFilter)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
				{
					if (OutTooltipText.IsEmpty())
					{
						OutTooltipText = FailureReason.ToString();
					}
					continue;
				}
			}

			OutTooltipText.Reset();
			OutOkIcon = true;
			return;
		}
	}
}

void USoundCueGraphSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);

	//////////////////////////////////////////////////////////////////////////
	// Handle dropped USoundWaves
	TArray<USoundWave*> Waves;
	for (const FAssetData& AssetData : Assets)
	{
		USoundWave* SoundWave = Cast<USoundWave>(AssetData.GetAsset());
		if (SoundWave)
		{
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(AssetData))
			{
				Waves.Add(SoundWave);
			}
		}
	}

	if (Waves.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("SoundCueEditorDropWave", "Sound Cue Editor: Drag and Drop Sound Wave") );

		USoundCueGraph* SoundCueGraph = CastChecked<USoundCueGraph>(Graph);
		USoundCue* SoundCue = SoundCueGraph->GetSoundCue();

		SoundCueGraph->Modify();

		TArray<USoundNode*> CreatedPlayers;
		FSoundCueEditorUtilities::CreateWaveContainers(Waves, SoundCue, CreatedPlayers, FDeprecateSlateVector2D(GraphPosition));
	}

	//////////////////////////////////////////////////////////////////////////
	// Handle dropped UDialogueWaves
	TArray<UDialogueWave*> Dialogues;
	for (const FAssetData& AssetData : Assets)
	{
		UDialogueWave* DialogueWave = Cast<UDialogueWave>(AssetData.GetAsset());
		if (DialogueWave)
		{
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(AssetData))
			{
				Dialogues.Add(DialogueWave);
			}
		}
	}

	if (Dialogues.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("SoundCueEditorDropDialogue", "Sound Cue Editor: Drag and Drop Dialogue Wave"));

		USoundCueGraph* SoundCueGraph = CastChecked<USoundCueGraph>(Graph);
		USoundCue* SoundCue = SoundCueGraph->GetSoundCue();

		SoundCueGraph->Modify();

		TArray<USoundNode*> CreatedPlayers;
		FSoundCueEditorUtilities::CreateDialogueContainers(Dialogues, SoundCue, CreatedPlayers, FDeprecateSlateVector2D(GraphPosition));
	}
}

void USoundCueGraphSchema::DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	// Currently, drag and drop is only supported for dropping on sound cue graph nodes, and in particular, sound wave players and sound dialogue players.
	if (!Node->IsA<USoundCueGraphNode>())
	{
		return;
	}

	USoundCueGraphNode* SoundCueGraphNode = CastChecked<USoundCueGraphNode>(Node);
	USoundCueGraph* SoundCueGraph = CastChecked<USoundCueGraph>(Node->GetGraph());
	USoundCue* SoundCue = SoundCueGraph->GetSoundCue();

	TArray<USoundWave*> Waves;
	TArray<UDialogueWave*> Dialogues;
	for (int32 AssetIdx = 0; AssetIdx < Assets.Num(); ++AssetIdx)
	{
		USoundWave* SoundWav = Cast<USoundWave>(Assets[AssetIdx].GetAsset());
		if (SoundWav)
		{
			Waves.Add(SoundWav);
		}
		else
		{
			UDialogueWave* Dialogue = Cast<UDialogueWave>(Assets[AssetIdx].GetAsset());
			if (Dialogue)
			{
				Dialogues.Add(Dialogue);
			}
		}
	}

	USoundNodeWavePlayer* SoundNodeWavePlayer = Cast<USoundNodeWavePlayer>(SoundCueGraphNode->SoundNode);
	if (SoundNodeWavePlayer != nullptr)
	{
		if (Waves.Num() > 0)
		{
			if (Waves.Num() >= 1)
			{
				SoundCueGraph->Modify();
				SoundNodeWavePlayer->SetSoundWave(Waves[0]);
			}

			for (int32 Index = 1; Index < Waves.Num(); Index++)
			{
				TArray<USoundNode*> CreatedPlayers;
				FSoundCueEditorUtilities::CreateWaveContainers(Waves, SoundCue, CreatedPlayers, FDeprecateSlateVector2D(GraphPosition));
			}
		}
		else if (Dialogues.Num() > 0)
		{
			TArray<USoundNode*> CreatedPlayers;
			FSoundCueEditorUtilities::CreateDialogueContainers(Dialogues, SoundCue, CreatedPlayers, FDeprecateSlateVector2D(GraphPosition));

			if (CreatedPlayers.Num() > 0)
			{
				USoundNode* OldNode = SoundCueGraphNode->SoundNode;
				SoundCueGraphNode->SetSoundNode(CreatedPlayers[0]);

				// Make sure SoundCue is updated to match graph
				SoundCue->CompileSoundNodesFromGraphNodes();

				// Remove this node from the SoundCue's list of all SoundNodes
				SoundCue->AllNodes.Remove(OldNode);
				SoundCue->MarkPackageDirty();
			}
		}
	}

	USoundNodeDialoguePlayer* SoundNodeDialoguePlayer = Cast<USoundNodeDialoguePlayer>(SoundCueGraphNode->SoundNode);
	if (SoundNodeDialoguePlayer != nullptr)
	{
		if (Dialogues.Num() > 0)
		{
			if (Dialogues.Num() >= 1)
			{
				SoundCueGraph->Modify();
				SoundNodeDialoguePlayer->SetDialogueWave(Dialogues[0]);

				if (Dialogues[0]->ContextMappings.Num() == 1)
				{
					SoundNodeDialoguePlayer->DialogueWaveParameter.Context.Speaker = Dialogues[0]->ContextMappings[0].Context.Speaker;
					SoundNodeDialoguePlayer->DialogueWaveParameter.Context.Targets = Dialogues[0]->ContextMappings[0].Context.Targets;
				}
			}

			for (int32 Index = 1; Index < Waves.Num(); Index++)
			{
				TArray<USoundNode*> CreatedPlayers;
				FSoundCueEditorUtilities::CreateDialogueContainers(Dialogues, SoundCue, CreatedPlayers, FDeprecateSlateVector2D(GraphPosition));
			}
		}
		else if (Waves.Num() > 0)
		{
			TArray<USoundNode*> CreatedPlayers;
			FSoundCueEditorUtilities::CreateWaveContainers(Waves, SoundCue, CreatedPlayers, FDeprecateSlateVector2D(GraphPosition));

			if (CreatedPlayers.Num() > 0)
			{
				USoundNode* OldNode = SoundCueGraphNode->SoundNode;
				SoundCueGraphNode->SetSoundNode(CreatedPlayers[0]);

				// Make sure SoundCue is updated to match graph
				SoundCue->CompileSoundNodesFromGraphNodes();

				// Remove this node from the SoundCue's list of all SoundNodes
				SoundCue->AllNodes.Remove(OldNode);
				SoundCue->MarkPackageDirty();
			}
		}
	}

	SoundCueGraph->NotifyGraphChanged();
}

void USoundCueGraphSchema::GetAllSoundNodeActions(FGraphActionMenuBuilder& ActionMenuBuilder, bool bShowSelectedActions) const
{
	FText SelectedItemText;
	bool IsSoundWaveSelected = false;
	bool IsDialogueWaveSelected = false;

	if (bShowSelectedActions)
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		// Get display text for any items that may be selected
		if (ActionMenuBuilder.FromPin == NULL)
		{
			TArray<USoundWave*> SelectedWavs;
			TArray<UDialogueWave*> SelectedDialogues;
			GEditor->GetSelectedObjects()->GetSelectedObjects<USoundWave>(SelectedWavs);
			GEditor->GetSelectedObjects()->GetSelectedObjects<UDialogueWave>(SelectedDialogues);

			int32 TotalWavs = SelectedWavs.Num() + SelectedDialogues.Num() ;

			if (TotalWavs > 1)
			{
				SelectedItemText = LOCTEXT("MultipleWAVsSelected", "Multiple WAVs");
			}
			else if (SelectedWavs.Num() == 1)
			{
				SelectedItemText = FText::FromString(SelectedWavs[0]->GetName());
				IsSoundWaveSelected = true;
			}
			else if (SelectedDialogues.Num() == 1)
			{
				SelectedItemText = FText::FromString(SelectedDialogues[0]->GetName());
				IsDialogueWaveSelected = true;
			}
		}
		else
		{
			USoundWave* SelectedWave = GEditor->GetSelectedObjects()->GetTop<USoundWave>();
			if (SelectedWave && ActionMenuBuilder.FromPin->Direction == EGPD_Input)
			{
				SelectedItemText = FText::FromString(SelectedWave->GetName());
				IsSoundWaveSelected = true;
			}
			else
			{
				UDialogueWave* SelectedDialogue = GEditor->GetSelectedObjects()->GetTop<UDialogueWave>();
				if (SelectedDialogue && ActionMenuBuilder.FromPin->Direction == EGPD_Input)
				{
					SelectedItemText = FText::FromString(SelectedDialogue->GetName());
					IsDialogueWaveSelected = true;
				}
			}
		}

		bShowSelectedActions = !SelectedItemText.IsEmpty();
	}
	
	for (TSubclassOf<USoundNode> SoundNodeClass : AllowedSoundNodes)
	{
		const USoundNode* SoundNode = SoundNodeClass->GetDefaultObject<USoundNode>();

		// when dragging from an output pin you can create anything but a wave player
		if (!ActionMenuBuilder.FromPin || ActionMenuBuilder.FromPin->Direction == EGPD_Input || SoundNode->GetMaxChildNodes() > 0)
		{
			const FText Name = FText::FromString(SoundNodeClass->GetDescription());

			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), Name);
				const FText AddToolTip = FText::Format(LOCTEXT("NewSoundCueNodeTooltip", "Adds {Name} node here"), Arguments);
				TSharedPtr<FSoundCueGraphSchemaAction_NewNode> NewNodeAction(new FSoundCueGraphSchemaAction_NewNode(LOCTEXT("SoundNodeAction", "Sound Node"), Name, AddToolTip, 0));
				ActionMenuBuilder.AddAction(NewNodeAction);
				NewNodeAction->SoundNodeClass = SoundNodeClass;
			}

			if (bShowSelectedActions && 
				(SoundNode->GetMaxChildNodes() == USoundNode::MAX_ALLOWED_CHILD_NODES || 
				   ((SoundNodeClass == USoundNodeWavePlayer::StaticClass() && IsSoundWaveSelected) || 
				    (SoundNodeClass == USoundNodeDialoguePlayer::StaticClass() && IsDialogueWaveSelected))))
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), Name);
				Arguments.Add(TEXT("SelectedItems"), SelectedItemText);
				const FText MenuDesc = FText::Format(LOCTEXT("NewSoundNodeRandom", "{Name}: {SelectedItems}"), Arguments);
				const FText ToolTip = FText::Format(LOCTEXT("NewSoundNodeRandomTooltip", "Adds a {Name} node for {SelectedItems} here"), Arguments);
				TSharedPtr<FSoundCueGraphSchemaAction_NewFromSelected> NewNodeAction(new FSoundCueGraphSchemaAction_NewFromSelected(LOCTEXT("FromSelected", "From Selected"),
					MenuDesc,
					ToolTip, 0));
				ActionMenuBuilder.AddAction(NewNodeAction);
				NewNodeAction->SoundNodeClass = (SoundNodeClass == USoundNodeWavePlayer::StaticClass() || SoundNodeClass == USoundNodeDialoguePlayer::StaticClass() ? NULL : SoundNodeClass);
			}
		}
	}
}

void USoundCueGraphSchema::GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	if (!ActionMenuBuilder.FromPin)
	{
		const bool bIsManyNodesSelected = CurrentGraph ? (FSoundCueEditorUtilities::GetNumberOfSelectedNodes(CurrentGraph) > 0) : false;
		const FText MenuDescription = bIsManyNodesSelected ? LOCTEXT("CreateCommentAction", "Create Comment from Selection") : LOCTEXT("AddCommentAction", "Add Comment...");
		const FText ToolTip = LOCTEXT("CreateCommentToolTip", "Creates a comment.");

		TSharedPtr<FSoundCueGraphSchemaAction_NewComment> NewAction(new FSoundCueGraphSchemaAction_NewComment(FText::GetEmpty(), MenuDescription, ToolTip, 0));
		ActionMenuBuilder.AddAction( NewAction );
	}
}
int32 USoundCueGraphSchema::GetNodeSelectionCount(const UEdGraph* Graph) const
{
	return FSoundCueEditorUtilities::GetNumberOfSelectedNodes(Graph);
}

TSharedPtr<FEdGraphSchemaAction> USoundCueGraphSchema::GetCreateCommentAction() const
{
	return TSharedPtr<FEdGraphSchemaAction>(static_cast<FEdGraphSchemaAction*>(new FSoundCueGraphSchemaAction_NewComment));
}

TSharedPtr<IAssetReferenceFilter> USoundCueGraphSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (const USoundCueGraph* SoundCueGraph = Cast<const USoundCueGraph>(Graph))
	{
		if (const USoundCue* SoundCue = SoundCueGraph->GetSoundCue())
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(SoundCue);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
