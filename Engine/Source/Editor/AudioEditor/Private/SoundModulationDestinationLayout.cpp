// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundModulationDestinationLayout.h"

#include "Algo/AnyOf.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioEditorModule.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioModulation.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Sound/SoundModulationDestination.h"
#include "Sound/SoundSourceBus.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "SoundModulationParameter"

namespace ModDestinationLayoutUtils
{
	IAudioModulationManager* GetEditorModulationManager()
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice();
				if (AudioDeviceHandle.IsValid() && AudioDeviceHandle->IsModulationPluginEnabled())
				{
					return AudioDeviceHandle->ModulationInterface.Get();
				}
			}
		}

		return nullptr;
	}

	bool IsModulationEnabled()
	{
		return GetEditorModulationManager() != nullptr;
	}

	FName GetParameterNameFromMetaData(const TSharedRef<IPropertyHandle>& InHandle)
	{
		static const FName AudioParamFieldName("AudioParam");

		if (InHandle->HasMetaData(AudioParamFieldName))
		{
			FString ParamString = InHandle->GetMetaData(AudioParamFieldName);
			return FName(ParamString);
		}

		return FName();
	}

	FName GetParameterClassFromMetaData(const TSharedRef<IPropertyHandle>& InHandle)
	{
		static const FName AudioParamClassFieldName("AudioParamClass");

		if (InHandle->HasMetaData(AudioParamClassFieldName))
		{
			const FName Param = *(InHandle->GetMetaData(AudioParamClassFieldName));
			return Param;
		}

		return FName();
	}

	bool IsParamMismatched(TSharedRef<IPropertyHandle> ModulatorHandle, TSharedRef<IPropertyHandle> StructPropertyHandle, FName* OutModParamClassName = nullptr, FName* OutDestParamClassName = nullptr, FName* OutModName = nullptr)
	{
		if (OutModParamClassName)
		{
			*OutModParamClassName = FName();
		}

		if (OutDestParamClassName)
		{
			*OutDestParamClassName = FName();
		}

		if (OutModName)
		{
			*OutModName = FName();
		}

		UObject* ModObject = nullptr;
		ModulatorHandle->GetValue(ModObject);

		USoundModulatorBase* ModBase = Cast<USoundModulatorBase>(ModObject);
		if (!ModBase)
		{
			return false;
		}

		const FName ModParamClassName = ModBase->GetOutputParameter().ClassName;
		const FName DestParamClassName = ModDestinationLayoutUtils::GetParameterClassFromMetaData(StructPropertyHandle);
		if (!ModParamClassName.IsNone() && !DestParamClassName.IsNone() && ModParamClassName != DestParamClassName)
		{
			if (OutModParamClassName)
			{
				*OutModParamClassName = ModParamClassName;
			}

			if (OutDestParamClassName)
			{
				*OutDestParamClassName = DestParamClassName;
			}

			if (OutModName)
			{
				*OutModName = ModBase->GetFName();
			}

			return true;
		}

		return false;
	}

	struct FNumericRanges
	{
		TOptional<float> ClampMinValue;
		TOptional<float> ClampMaxValue;
		TOptional<float> UIMinValue;
		TOptional<float> UIMaxValue;
	};

	FText SetMetaData(TSharedRef<IPropertyHandle> StructPropertyHandle, TSharedRef<IPropertyHandle> ValueHandle, FText& OutUnitDisplayText, FName& OutParamName, FNumericRanges& OutRanges)
	{
		bool bClampValuesSet = false;
		float ClampMinValue = 0.0f;
		float ClampMaxValue = 1.0f;
		float UIMinValue = 0.0f;
		float UIMaxValue = 1.0f;
		if (StructPropertyHandle->HasMetaData("ClampMin"))
		{
			bClampValuesSet = true;
			FString ParamString = StructPropertyHandle->GetMetaData("ClampMin");
			ClampMinValue = FCString::Atof(*ParamString);
		}

		if (StructPropertyHandle->HasMetaData("ClampMax"))
		{
			FString ParamString = StructPropertyHandle->GetMetaData("ClampMax");
			ClampMaxValue = FCString::Atof(*ParamString);
			bClampValuesSet = true;
		}

		OutParamName = ModDestinationLayoutUtils::GetParameterNameFromMetaData(StructPropertyHandle);
		if (OutParamName != FName())
		{
			// If parameter was provided, it overrides ClampMin/Max.
			if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(OutParamName))
			{
				// if no valid parameter was found & the user has specified their own clamping, don't override it with the default parameter range of [0,1]
				if (false == (Parameter->ParameterName == FName() && bClampValuesSet))
				{
					UIMinValue = Parameter->MinValue;
					UIMaxValue = Parameter->MaxValue;
					ClampMinValue = UIMinValue;
					ClampMaxValue = UIMaxValue;
					OutUnitDisplayText = Parameter->UnitDisplayName;
					if (bClampValuesSet)
					{
						UE_LOGF(LogAudioEditor, Verbose, "ClampMin/Max overridden by AudioModulation plugin asset with ParamName '%ls'.", *OutParamName.ToString());
					}
				}
			}
		}

		// User data overrides UIMin/Max if its in clamp range.
		if (StructPropertyHandle->HasMetaData("UIMin"))
		{
			float NewMin = UIMinValue;
			FString ParamString = StructPropertyHandle->GetMetaData("UIMin");
			NewMin = FCString::Atof(*ParamString);
			UIMinValue = FMath::Clamp(NewMin, ClampMinValue, ClampMaxValue);
		}

		if (StructPropertyHandle->HasMetaData("UIMax"))
		{
			float NewMax = UIMaxValue;
			FString ParamString = StructPropertyHandle->GetMetaData("UIMax");
			NewMax = FCString::Atof(*ParamString);
			UIMaxValue = FMath::Clamp(NewMax, ClampMinValue, ClampMaxValue);
		}

		ValueHandle->SetInstanceMetaData("ClampMin", FString::Printf(TEXT("%f"), ClampMinValue));
		ValueHandle->SetInstanceMetaData("ClampMax", FString::Printf(TEXT("%f"), ClampMaxValue));
		ValueHandle->SetInstanceMetaData("UIMin", FString::Printf(TEXT("%f"), UIMinValue));
		ValueHandle->SetInstanceMetaData("UIMax", FString::Printf(TEXT("%f"), UIMaxValue));
		
		OutRanges.ClampMinValue = ClampMinValue;
		OutRanges.ClampMaxValue = ClampMaxValue;
		OutRanges.UIMinValue = UIMinValue;
		OutRanges.UIMaxValue = UIMaxValue;

		return OutUnitDisplayText;
	}

	struct FValueEditState
	{
		TUniquePtr<FScopedTransaction> Transaction;
		bool bIsDragging = false;
		float LastCommittedValue = 0.f;
	};

	// Helper: modify all outer objects
	static void ModifyOuterObjects(TSharedRef<IPropertyHandle> Handle)
	{
		TArray<UObject*> OuterObjects;
		Handle->GetOuterObjects(OuterObjects);
		for (UObject* Obj : OuterObjects)
		{
			if (Obj)
			{
				Obj->Modify();
			}
		}
	}

	void CustomizeChildren_AddValueRow(
		IDetailChildrenBuilder& ChildBuilder,
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		TSharedRef<IPropertyHandle> ValueHandle,
		TSharedRef<IPropertyHandle> ModulatorsHandle,
		TSharedRef<IPropertyHandle> EnablementHandle)
	{
		FText UnitDisplayText = FText::GetEmpty();
		FName ParamName;
		FNumericRanges Ranges;
		SetMetaData(StructPropertyHandle, ValueHandle, UnitDisplayText, ParamName, Ranges);

		const FText DisplayName = StructPropertyHandle->GetPropertyDisplayName();

		// Keep state per-row (shared so lambdas stay valid)
		TSharedRef<FValueEditState> EditState = MakeShared<FValueEditState>();

		auto GetCurrentValue = [ValueHandle]() -> TOptional<float>
		{
			float V = 0.0f;
			if (ValueHandle->GetValue(V) == FPropertyAccess::Success)
			{
				return V;
			}
			return TOptional<float>();
		};

		auto SetValueInteractive = [ValueHandle](float NewValue)
		{
			ValueHandle->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange);
		};

		auto SetValueCommitted = [ValueHandle](float NewValue)
		{
			ValueHandle->SetValue(NewValue, EPropertyValueSetFlags::DefaultFlags);
		};

		auto BeginTransaction = [EditState, ValueHandle]()
		{
			if (!EditState->Transaction)
			{
				EditState->Transaction = MakeUnique<FScopedTransaction>(
					NSLOCTEXT("Audio Modulation", "ChangeModulationValue", "Change Modulation Value"));

				// Record the outer objects into the transaction buffer so we can undo these changes
				ModifyOuterObjects(ValueHandle);
			}
		};

		auto EndTransaction = [EditState]()
		{
			EditState->Transaction.Reset();
			EditState->bIsDragging = false;
		};

		ChildBuilder.AddCustomRow(DisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(DisplayName)
			.ToolTipText(StructPropertyHandle->GetToolTipText())
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<float>)
					.Value_Lambda(GetCurrentValue)
					.MinValue(Ranges.ClampMinValue)
					.MaxValue(Ranges.ClampMaxValue)
					.MinSliderValue(Ranges.UIMinValue)
					.MaxSliderValue(Ranges.UIMaxValue)
					.AllowSpin(true)
					.Font(IDetailLayoutBuilder::GetDetailFont())

					// Drag start: open transaction & Modify()
					.OnBeginSliderMovement_Lambda([EditState, BeginTransaction]()
					{
						EditState->bIsDragging = true;
						BeginTransaction();
					})

					// Drag update: interactive set (keeps responsive updates)
					.OnValueChanged_Lambda([EditState, BeginTransaction, SetValueInteractive](float NewValue)
					{
						// OnValueChanged can fire from typing too; only treat as interactive if dragging.
						if (EditState->bIsDragging)
						{
							BeginTransaction();
							SetValueInteractive(NewValue);
						}
					})

					// Drag end: final committed write + close transaction
					.OnEndSliderMovement_Lambda([EditState, BeginTransaction, SetValueCommitted, EndTransaction](float NewValue)
					{
						BeginTransaction();
						SetValueCommitted(NewValue);
						EndTransaction();
					})

					// Text commit (enter/tab/click away): commit write + close transaction
					.OnValueCommitted_Lambda([BeginTransaction, SetValueCommitted, EndTransaction](float NewValue, ETextCommit::Type)
					{
						BeginTransaction();
						SetValueCommitted(NewValue);
						EndTransaction();
					})
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(UnitDisplayText)
					.ToolTipText(ValueHandle->GetToolTipText())
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					EnablementHandle->CreatePropertyValueWidget()
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					EnablementHandle->CreatePropertyNameWidget()
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("ResetToParameterDefaultToolTip", "Reset to parameter's default"))
					.ButtonStyle(FAppStyle::Get(), TEXT("NoBorder"))
					.ContentPadding(0.0f)
					.Visibility(TAttribute<EVisibility>::Create([ParamName, ValueHandle]
					{
						float CurrentValue = 0.0f;
						ValueHandle->GetValue(CurrentValue);

						const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(ParamName);
						if (Parameter && Parameter->DefaultValue == CurrentValue)
						{
							return EVisibility::Hidden;
						}
						return EVisibility::Visible;
					}))
					.OnClicked(FOnClicked::CreateLambda([ParamName, ValueHandle]()
					{
						if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(ParamName))
						{
							ValueHandle->SetValue(Parameter->DefaultValue);
						}
						return FReply::Handled();
					}))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
					]
				]
		];

		EnablementHandle->SetOnPropertyValueChanged(
			FSimpleDelegate::CreateLambda([EnablementHandle, ValueHandle, StructPropertyHandle, ModulatorsHandle, BeginTransaction, EndTransaction]()
		{
			BeginTransaction();

			bool bEnabled = false;
			EnablementHandle->GetValue(bEnabled);
			if (bEnabled)
			{
				const FName ParamName = ModDestinationLayoutUtils::GetParameterNameFromMetaData(StructPropertyHandle);
				if (const Audio::FModulationParameter* Parameter = Audio::GetModulationParameterPtr(ParamName))
				{
					ValueHandle->SetValue(Parameter->DefaultValue);
					EndTransaction();
					return;
				}
			}
			ModulatorsHandle->AsSet()->Empty();

			EndTransaction();
		}));
	}

	void CustomizeChildren_AddValueNoModRow(IDetailChildrenBuilder& ChildBuilder, TSharedRef<IPropertyHandle> StructPropertyHandle, TSharedRef<IPropertyHandle> ValueHandle)
	{
		FText UnitDisplayText = FText::GetEmpty();
		FName ParamName;
		FNumericRanges Ranges;
		SetMetaData(StructPropertyHandle, ValueHandle, UnitDisplayText, ParamName, Ranges);

		const FText DisplayName = StructPropertyHandle->GetPropertyDisplayName();
		FDetailWidgetRow& ValueNoModRow = ChildBuilder.AddCustomRow(DisplayName);
		ValueNoModRow.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(DisplayName)
			.ToolTipText(StructPropertyHandle->GetToolTipText())
		]
		.ValueContent()
		.MinDesiredWidth(120.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					ValueHandle->CreatePropertyValueWidget()
				]
		];
	}

	void CustomizeChildren_AddModulatorRow(IDetailChildrenBuilder& ChildBuilder, TSharedRef<IPropertyHandle> StructPropertyHandle, TSharedRef<IPropertyHandle> ModulatorsHandle, TSharedRef<IPropertyHandle> EnablementHandle)
	{
		const FText DisplayName = StructPropertyHandle->GetPropertyDisplayName();

		auto ModEnabled = [EnablementHandle]()
		{
			bool bModulationEnabled = false;
			EnablementHandle->GetValue(bModulationEnabled);
			return bModulationEnabled
				? EVisibility::Visible
				: EVisibility::Collapsed;
		};

		const TAttribute<EVisibility> ModulatorVisibility = TAttribute<EVisibility>::Create(ModEnabled);

		ChildBuilder.AddProperty(ModulatorsHandle)
			.DisplayName(FText::Format(LOCTEXT("SoundModulationParameter_ModulatorsFormat", "{0} Modulators"), DisplayName))
			.Visibility(ModulatorVisibility);

		ChildBuilder.AddCustomRow(LOCTEXT("SoundModulationDestinationLayout_UnitMismatchHeadingWarning", "Unit Mismatch Warning"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(FText::Format(LOCTEXT("ModulationDestinationLayout_UnitMismatchHeader", "{0} Mismatched Parameters"), DisplayName))
				.ToolTipText(StructPropertyHandle->GetToolTipText())
			]
			.ValueContent()
			.MinDesiredWidth(150.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f, 3.0f, 0.0f, 3.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
						.Text(TAttribute<FText>::Create([ModSet = ModulatorsHandle->AsSet(), StructPropertyHandle]()
						{
							uint32 NumElements = 0;
							ModSet->GetNumElements(NumElements);

							FString MismatchedText("");
							for (int32 i = 0; i < (int32)NumElements; ++i)
							{
								TSharedRef<IPropertyHandle> SetMemberHandle = ModSet->GetElement(i);
								FName ModParamClassName;
								FName DestClassName;
								FName ModName;
								if (ModDestinationLayoutUtils::IsParamMismatched(SetMemberHandle, StructPropertyHandle, &ModParamClassName, &DestClassName, &ModName))
								{
									MismatchedText += FText::Format(
										LOCTEXT("ModulationDestinationLayout_UnitMismatchesFormat", "{0} ({1}), Expected: {2}\n"),
										FText::FromName(ModName),
										FText::FromName(ModParamClassName),
										FText::FromName(DestClassName)
									).ToString();
								}
							}

							return FText::FromString(MismatchedText);
						}))
					]
			]
			.Visibility(TAttribute<EVisibility>::Create([ModSet = ModulatorsHandle->AsSet(), ModEnabled, StructPropertyHandle]()
			{
				EVisibility Visibility = ModEnabled();
				if (Visibility == EVisibility::Collapsed)
				{
					return Visibility;
				}

				uint32 NumElements = 0;
				ModSet->GetNumElements(NumElements);

				bool bIsMismatched = false;
				for (int32 i = 0; i < (int32)NumElements; ++i)
				{
					TSharedRef<IPropertyHandle> SetMemberHandle = ModSet->GetElement(i);
					if (ModDestinationLayoutUtils::IsParamMismatched(SetMemberHandle, StructPropertyHandle))
					{
						bIsMismatched = true;
					}
				}

				return bIsMismatched ? EVisibility::Visible : EVisibility::Hidden;
			}));
	}

	void SetBoundsMetaData(FName FieldName, float InDefault, const TSharedRef<IPropertyHandle>& InHandle, TSharedRef<IPropertyHandle>& OutHandle)
	{
		if (InHandle->HasMetaData(FieldName))
		{
			const FString& Value = InHandle->GetMetaData(FieldName);
			OutHandle->SetInstanceMetaData(FieldName, Value);
		}
		else
		{
			OutHandle->SetInstanceMetaData(FieldName, FString::Printf(TEXT("%f"), InDefault));
		}
	}
} // namespace ModParamLayoutUtils

void FSoundModulationDestinationLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FSoundModulationDestinationLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	TSharedRef<IPropertyHandle>EnablementHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDestinationSettings, bEnableModulation)).ToSharedRef();
	TSharedRef<IPropertyHandle>ModulatorsHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDestinationSettings, Modulators)).ToSharedRef();
	TSharedRef<IPropertyHandle>ValueHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDestinationSettings, Value)).ToSharedRef();

	if (ModDestinationLayoutUtils::IsModulationEnabled())
	{
		ModDestinationLayoutUtils::CustomizeChildren_AddValueRow(ChildBuilder, StructPropertyHandle, ValueHandle, ModulatorsHandle, EnablementHandle);
		ModDestinationLayoutUtils::CustomizeChildren_AddModulatorRow(ChildBuilder, StructPropertyHandle, ModulatorsHandle, EnablementHandle);
	}
	else
	{
		ModDestinationLayoutUtils::CustomizeChildren_AddValueNoModRow(ChildBuilder, StructPropertyHandle, ValueHandle);
	}
}

void FSoundModulationDefaultSettingsLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FSoundModulationDefaultSettingsLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (ModDestinationLayoutUtils::IsModulationEnabled())
	{
		TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}

		TSharedRef<IPropertyHandle> VolumeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultSettings, VolumeModulationDestination)).ToSharedRef();
		TSharedRef<IPropertyHandle> PitchHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultSettings, PitchModulationDestination)).ToSharedRef();
		TSharedRef<IPropertyHandle> HighpassHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultSettings, HighpassModulationDestination)).ToSharedRef();
		TSharedRef<IPropertyHandle> LowpassHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultSettings, LowpassModulationDestination)).ToSharedRef();

		ChildBuilder.AddProperty(VolumeHandle);

		// Do not add the pitch destination to source buses
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);
		const bool bEditingSourceBus = Algo::AnyOf(Objects, [](const UObject* Object) -> bool
			{
				return Object->IsA<USoundSourceBus>();
			});
		if (!bEditingSourceBus)
		{
			ChildBuilder.AddProperty(PitchHandle);
		}
		
		ChildBuilder.AddProperty(HighpassHandle);
		ChildBuilder.AddProperty(LowpassHandle);
	}
}

void FSoundModulationDefaultRoutingSettingsLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FSoundModulationDefaultRoutingSettingsLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (ModDestinationLayoutUtils::IsModulationEnabled())
	{
		TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}

		TSharedRef<IPropertyHandle> VolumeRouting = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, VolumeRouting)).ToSharedRef();
		TSharedRef<IPropertyHandle> VolumeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, VolumeModulationDestination)).ToSharedRef();

		TSharedRef<IPropertyHandle> PitchRouting = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, PitchRouting)).ToSharedRef();
		TSharedRef<IPropertyHandle> PitchHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, PitchModulationDestination)).ToSharedRef();

		TSharedRef<IPropertyHandle> HighpassRouting = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, HighpassRouting)).ToSharedRef();
		TSharedRef<IPropertyHandle> HighpassHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, HighpassModulationDestination)).ToSharedRef();

		TSharedRef<IPropertyHandle> LowpassRouting = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, LowpassRouting)).ToSharedRef();
		TSharedRef<IPropertyHandle> LowpassHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundModulationDefaultRoutingSettings, LowpassModulationDestination)).ToSharedRef();

		auto ShowModSettings = [] (TSharedRef<IPropertyHandle> RoutingHandle)
		{
			return TAttribute<EVisibility>::Create([RoutingHandle]()
			{
				uint8 RoutingValue = 0;
				if (RoutingHandle->GetValue(RoutingValue) != FPropertyAccess::Success)
				{
					return EVisibility::Collapsed;
				}

				switch (static_cast<EModulationRouting>(RoutingValue))
				{
					case EModulationRouting::Disable:
					case EModulationRouting::Inherit:
					{
						return EVisibility::Collapsed;
					}

					case EModulationRouting::Override:
					case EModulationRouting::Union:
					default:
					{
						return EVisibility::Visible;
					}
				}
			});
		};

		ChildBuilder.AddProperty(VolumeRouting);
		ChildBuilder.AddProperty(VolumeHandle).Visibility(ShowModSettings(VolumeRouting));

		// Do not add pitch destination to source buses
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);
		const bool bEditingSourceBus = Algo::AnyOf(Objects, [](const UObject* Object) -> bool
			{
				return Object->IsA<USoundSourceBus>();
			});
		if (!bEditingSourceBus)
		{
			ChildBuilder.AddProperty(PitchRouting);
			ChildBuilder.AddProperty(PitchHandle).Visibility(ShowModSettings(PitchRouting));
		}

		ChildBuilder.AddProperty(HighpassRouting);
		ChildBuilder.AddProperty(HighpassHandle).Visibility(ShowModSettings(HighpassRouting));
		ChildBuilder.AddProperty(LowpassRouting);
		ChildBuilder.AddProperty(LowpassHandle).Visibility(ShowModSettings(LowpassRouting));
	}
}
#undef LOCTEXT_NAMESPACE
