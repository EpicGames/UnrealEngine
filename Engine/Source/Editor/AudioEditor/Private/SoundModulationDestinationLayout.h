// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

class FSoundModulationDestinationLayoutCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSoundModulationDestinationLayoutCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};

class FSoundModulationDefaultSettingsLayoutCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSoundModulationDefaultSettingsLayoutCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};

class FSoundModulationDefaultRoutingSettingsLayoutCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSoundModulationDefaultRoutingSettingsLayoutCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};