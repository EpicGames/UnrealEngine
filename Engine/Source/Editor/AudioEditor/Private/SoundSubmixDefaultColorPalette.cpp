// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixDefaultColorPalette.h"

#include "Misc/AssertionMacros.h"
#include "Sound/SoundSubmix.h"

FName Audio::GetNameForSubmixType(const USoundSubmixBase* InSubmix)
{
	if (!ensure(InSubmix))
	{
		return SoundSubmixName;
	}
	else if (InSubmix->IsA<USoundSubmix>())
	{
		return SoundSubmixName;
	}
	else if (InSubmix->IsA<USoundfieldSubmix>())
	{
		return SoundfieldSubmixName;
	}
	else if (InSubmix->IsA<UEndpointSubmix>())
	{
		return EndpointSubmixName;
	}
	else if (InSubmix->IsA<USoundfieldEndpointSubmix>())
	{
		return SoundfieldEndpointSubmixName;
	}
	else
	{
		return SoundSubmixName;
	}
}

FColor Audio::GetColorForSubmixType(const FName& InSubmixName)
{
	if (InSubmixName == SoundSubmixName)
	{
		return DefaultSubmixColor;
	}
	else if (InSubmixName == SoundfieldSubmixName)
	{
		return SoundfieldDefaultSubmixColor;
	}
	else if (InSubmixName == EndpointSubmixName)
	{
		return EndpointDefaultSubmixColor;
	}
	else if (InSubmixName == SoundfieldEndpointSubmixName)
	{
		return SoundfieldEndpointDefaultSubmixColor;
	}
	else
	{
		return DefaultSubmixColor;
	}
}

FColor Audio::GetColorForSubmixType(const USoundSubmixBase* InSubmix)
{
	if (!ensure(InSubmix))
	{
		return DefaultSubmixColor;
	}
	else if (InSubmix->IsA<USoundSubmix>())
	{
		return DefaultSubmixColor;
	}
	else if (InSubmix->IsA<USoundfieldSubmix>())
	{
		return SoundfieldDefaultSubmixColor;
	}
	else if (InSubmix->IsA<UEndpointSubmix>())
	{
		return EndpointDefaultSubmixColor;
	}
	else if (InSubmix->IsA<USoundfieldEndpointSubmix>())
	{
		return SoundfieldEndpointDefaultSubmixColor;
	}
	else
	{
		return DefaultSubmixColor;
	}
}

const bool Audio::IsConnectionPerformingSoundfieldConversion(const USoundSubmixBase* InputSubmix, const USoundSubmixBase* OutputSubmix)
{
	if (!ensure(InputSubmix && OutputSubmix))
	{
		return false;
	}
	else if (InputSubmix->IsA<USoundfieldSubmix>() || InputSubmix->IsA<USoundfieldEndpointSubmix>())
	{
		return OutputSubmix->IsA<USoundSubmix>() || OutputSubmix->IsA<UEndpointSubmix>();
	}
	else if (OutputSubmix->IsA<USoundfieldSubmix>() || OutputSubmix->IsA<USoundfieldEndpointSubmix>())
	{
		return InputSubmix->IsA<USoundSubmix>() || InputSubmix->IsA<UEndpointSubmix>();
	}
	else
	{
		return false;
	}
}
