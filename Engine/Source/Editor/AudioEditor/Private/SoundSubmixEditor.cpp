// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixEditor.h"

#include "AssetToolsModule.h"
#include "AudioEditorModule.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EngineAnalytics.h"
#include "Factories/SoundSubmixFactory.h"
#include "FileHelpers.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "GraphEditAction.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "IAnalyticsProviderET.h"
#include "IAssetTools.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SSoundSubmixActionMenu.h"
#include "ScopedTransaction.h"
#include "Sound/SoundSubmix.h"
#include "SoundSubmixGraph/SoundSubmixGraph.h"
#include "SoundSubmixGraph/SoundSubmixGraphNode.h"
#include "SoundSubmixGraph/SoundSubmixGraphSchema.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

class IToolkitHost;

#define LOCTEXT_NAMESPACE "SoundSubmixEditor"
DEFINE_LOG_CATEGORY_STATIC(LogSoundSubmixEditor, Log, All);

const FName FSoundSubmixEditor::GraphCanvasTabId(TEXT("SoundSubmixEditor_GraphCanvas"));
const FName FSoundSubmixEditor::PropertiesTabId(TEXT("SoundSubmixEditor_Properties"));


class SSoundSubmixGraphEditor : public SGraphEditor
{
private:
	TWeakPtr<FSoundSubmixEditor> SubmixEditor;

public:
	SSoundSubmixGraphEditor()
		: SGraphEditor()
		, SubmixEditor(nullptr)
	{
	}

	void Construct(const FArguments& InArgs, TSharedPtr<FSoundSubmixEditor> InEditor)
	{
		SubmixEditor = InEditor;
		SGraphEditor::Construct(InArgs);
	}

	void OnGraphChanged(const FEdGraphEditAction& InAction) override
	{
		if (SubmixEditor.IsValid())
		{
			TSharedPtr<FSoundSubmixEditor> PinnedEditor = SubmixEditor.Pin();
			if (InAction.Graph && InAction.Graph == PinnedEditor->GetGraph())
			{
				PinnedEditor->AddMissingEditableSubmixes();
				PinnedEditor->RefreshDetails();
			}
		}
		SGraphEditor::OnGraphChanged(InAction);
	}
};

void FSoundSubmixEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SoundSubmixEditor", "Sound Submix Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(GraphCanvasTabId, FOnSpawnTab::CreateSP(this, &FSoundSubmixEditor::SpawnTab_GraphCanvas))
		.SetDisplayName(LOCTEXT("GraphCanvasTab", "Graph"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FSoundSubmixEditor::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FSoundSubmixEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( GraphCanvasTabId );
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
}

void FSoundSubmixEditor::AddEditableSubmixChildren(USoundSubmixBase* ParentSubmix)
{
	if (!ParentSubmix)
	{
		return;
	}

	ParentSubmix->SetFlags(RF_Transactional);
	for (USoundSubmixBase* Child : ParentSubmix->ChildSubmixes)
	{
		if (Child)
		{
			// Only update the child submix if it's different
			// this avoids unnecessarily dirtying submixes if they aren't actually changed
			if (Child->SoundSubmixGraph != ParentSubmix->SoundSubmixGraph)
			{
				// Preserve child pacakge state for editor-only bookkeeping
				UPackage* ChildPkg = Child->GetOutermost();
				const bool bWasDirty = ChildPkg ? ChildPkg->IsDirty() : false;

				Child->Modify(false);
				Child->SoundSubmixGraph = ParentSubmix->SoundSubmixGraph;
				Child->PostEditChange();

				if (ChildPkg)
				{
					ChildPkg->SetDirtyFlag(bWasDirty);
				}
			}
			AddEditingObject(Child);
			AddEditableSubmixChildren(Child);
		}
	}
}

void FSoundSubmixEditor::Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit)
{
	USoundSubmixBase* SoundSubmix = CastChecked<USoundSubmixBase>(ObjectToEdit);

	USoundSubmixWithParentBase* SubmixWithParent = Cast<USoundSubmixWithParentBase>(SoundSubmix);
	while (SubmixWithParent && SubmixWithParent->ParentSubmix)
	{
		SoundSubmix = SubmixWithParent->ParentSubmix;
		SubmixWithParent = Cast<USoundSubmixWithParentBase>(SoundSubmix);
	}

	GEditor->RegisterForUndo(this);

	ToolkitCommands->MapAction
	(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &FSoundSubmixEditor::UndoGraphAction)
	);

	ToolkitCommands->MapAction
	(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &FSoundSubmixEditor::RedoGraphAction)
	);

	USoundSubmixGraph* SoundSubmixGraph = CastChecked<USoundSubmixGraph>(FBlueprintEditorUtils::CreateNewGraph(SoundSubmix, NAME_None, USoundSubmixGraph::StaticClass(), USoundSubmixGraphSchema::StaticClass()));
	SoundSubmixGraph->SetRootSoundSubmix(SoundSubmix);

	SoundSubmix->SoundSubmixGraph = SoundSubmixGraph;
	SoundSubmixGraph->RebuildGraph();

	CreateInternalWidgets(SoundSubmix);

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_SoundSubmixEditor_Layout_v4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->Split
				(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.2f)
					->AddTab(PropertiesTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.8f)
					->AddTab(GraphCanvasTabId, ETabState::OpenedTab)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TEXT("SoundSubmixEditorApp"), StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, SoundSubmix);

	// Must be called after super class initialization
	AddEditableSubmixChildren(SoundSubmix);

	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
	AddMenuExtender(AudioEditorModule->GetSoundSubmixMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	AddToolbarExtender(AudioEditorModule->GetSoundSubmixToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	GraphEditor->SelectAllNodes();
	for (UObject* SelectedNode : GraphEditor->GetSelectedNodes())
	{
		USoundSubmixGraphNode* GraphNode = CastChecked<USoundSubmixGraphNode>(SelectedNode);
		if (GraphNode->SoundSubmix == ObjectToEdit)
		{
			GraphEditor->ClearSelectionSet();
			GraphEditor->SetNodeSelection(GraphNode, true);
			DetailsView->SetObject(ObjectToEdit);
			break;
		}
	}
}

FSoundSubmixEditor::~FSoundSubmixEditor()
{
	GEditor->UnregisterForUndo(this);
	DetailsView.Reset();
}

void FSoundSubmixEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(GetEditingObjectPtrs());
}

TSharedRef<SDockTab> FSoundSubmixEditor::SpawnTab_GraphCanvas(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == GraphCanvasTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("GraphCanvasTitle", "Graph"))
		[
			GraphEditor.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FSoundSubmixEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PropertiesTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("SoundSubmixPropertiesTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return SpawnedTab;
}

FName FSoundSubmixEditor::GetToolkitFName() const
{
	return FName("SoundSubmixEditor");
}

FText FSoundSubmixEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Sound Submix Editor");
}

FText FSoundSubmixEditor::GetToolkitToolTipText() const
{
	return GetToolTipTextForObject(GetEditingObjects()[0]);
}

FString FSoundSubmixEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT( "WorldCentricTabPrefix", "Sound Submix " ).ToString();
}

FLinearColor FSoundSubmixEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.2f, 0.4f, 0.8f, 0.5f );
}

void FSoundSubmixEditor::CreateInternalWidgets(USoundSubmixBase* InSoundSubmix)
{
	GraphEditor = CreateGraphEditorWidget(InSoundSubmix);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(InSoundSubmix);
}

TSharedRef<SGraphEditor> FSoundSubmixEditor::CreateGraphEditorWidget(USoundSubmixBase* InSoundSubmix)
{
	if (!GraphEditorCommands.IsValid())
	{
		GraphEditorCommands = MakeShareable(new FUICommandList);

		// Editing commands
		GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP(this, &FSoundSubmixEditor::SelectAllNodes),
			FCanExecuteAction::CreateSP(this, &FSoundSubmixEditor::CanSelectAllNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &FSoundSubmixEditor::RemoveSelectedNodes),
			FCanExecuteAction::CreateSP(this, &FSoundSubmixEditor::CanRemoveNodes)
		);
	}

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_SoundSubmix", "SOUND SUBMIX");

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FSoundSubmixEditor::OnSelectedNodesChanged);
	InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FSoundSubmixEditor::OnCreateGraphActionMenu);

	return SNew(SSoundSubmixGraphEditor, SharedThis(this))
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(AppearanceInfo)
		.GraphToEdit(InSoundSubmix->SoundSubmixGraph)
		.GraphEvents(InEvents)
		.ShowGraphStateOverlay(false);
}

void FSoundSubmixEditor::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	if(NewSelection.Num() > 0)
	{
		TArray<UObject*> Selection;
		for (UObject* Obj : NewSelection)
		{
			USoundSubmixGraphNode* GraphNode = CastChecked<USoundSubmixGraphNode>(Obj);
			Selection.Add(GraphNode->SoundSubmix);
		}
		DetailsView->SetObjects(Selection);
	}
	else
	{
		DetailsView->SetObject(GetEditingObjects()[0]);
	}
}

FActionMenuContent FSoundSubmixEditor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	TSharedRef<SSoundSubmixActionMenu> ActionMenu = 
		SNew(SSoundSubmixActionMenu)
		.GraphObj(InGraph)
		.NewNodePosition(FDeprecateSlateVector2D(InNodePosition))
		.DraggedFromPins(InDraggedPins)
		.AutoExpandActionMenu(bAutoExpand)
		.OnClosedCallback(InOnMenuClosed);

	return FActionMenuContent( ActionMenu, ActionMenu );
}

void FSoundSubmixEditor::SelectAllNodes()
{
	GraphEditor->SelectAllNodes();
}

void FSoundSubmixEditor::SelectSubmixes(TSet<USoundSubmixBase*>& InSubmixes)
{
	TArray<UObject*> ObjectsToSelect;

	GraphEditor->SelectAllNodes();
	const TSet<UObject*> SelectedNodes = GraphEditor->GetSelectedNodes();
	GraphEditor->ClearSelectionSet();

	for (UObject* SelectedNode : SelectedNodes)
	{
		if (SelectedNode)
		{
			USoundSubmixGraphNode* GraphNode = CastChecked<USoundSubmixGraphNode>(SelectedNode);
			if (USoundSubmixBase* Submix = GraphNode->SoundSubmix)
			{
				if (InSubmixes.Contains(Submix))
				{
					ObjectsToSelect.Add(Submix);
					GraphEditor->SetNodeSelection(GraphNode, true /* bSelect */);
				}
			}
		}
	}

	DetailsView->SetObjects(ObjectsToSelect);
}

bool FSoundSubmixEditor::CanSelectAllNodes() const
{
	return true;
}

void FSoundSubmixEditor::RemoveSelectedNodes()
{
	const FScopedTransaction Transaction(LOCTEXT("SoundSubmixEditorRemoveSelectedNode", "Sound Submix Editor: Remove Selected SoundSubmixes from editor"));

	int32 NumObjectsRemoved = 0;

	const TSet<UObject*> SelectedNodes = GraphEditor->GetSelectedNodes();
	for (UObject* SelectedNode : SelectedNodes)
	{
		USoundSubmixGraphNode* Node = Cast<USoundSubmixGraphNode>(SelectedNode);
		if (Node && Node->SoundSubmix && Node->CanUserDeleteNode())
		{
			NumObjectsRemoved++;
			RemoveEditingObject(Node->SoundSubmix);
		}
	}

	if (NumObjectsRemoved > 0)
	{
		USoundSubmixGraph* Graph = CastChecked<USoundSubmixGraph>(GraphEditor->GetCurrentGraph());
		Graph->RecursivelyRemoveNodes(SelectedNodes);
		GraphEditor->ClearSelectionSet();
	}
}

bool FSoundSubmixEditor::CanRemoveNodes() const
{
	return GraphEditor->GetSelectedNodes().Num() > 0;
}

void FSoundSubmixEditor::UndoGraphAction()
{
	GEditor->UndoTransaction();
}

void FSoundSubmixEditor::RedoGraphAction()
{
	// Clear selection, to avoid holding refs to nodes that go away
	GraphEditor->ClearSelectionSet();

	GEditor->RedoTransaction();
}

void FSoundSubmixEditor::CreateSoundSubmix(UEdGraphPin* FromPin, const FVector2D Location, const FString& Name)
{
	if (Name.IsEmpty())
	{
		return;
	}

	for (UObject* Obj : GetEditingObjects())
	{
		if (Obj->GetName() == Name)
		return;
	}

	// Derive new package path from existing asset's path
	USoundSubmixBase* SoundSubmix = CastChecked<USoundSubmixBase>(GetEditingObjects()[0]);
	FString PackagePath = SoundSubmix->GetPathName();
	FString AssetName = FString::Printf(TEXT("/%s.%s"), *SoundSubmix->GetName(), *SoundSubmix->GetName());
	PackagePath.RemoveFromEnd(AssetName);

	// Create a sound submix factory to create a new sound class
	USoundSubmixFactory* SoundSubmixFactory = NewObject<USoundSubmixFactory>();

	// Load asset tools to create the asset properly
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	USoundSubmix* NewSoundSubmix = Cast<USoundSubmix>(AssetToolsModule.Get().CreateAsset(Name, PackagePath, USoundSubmix::StaticClass(), SoundSubmixFactory, FName("SoundSubmixEditorNewAsset")));

	if (NewSoundSubmix)
	{
		CastChecked<USoundSubmixGraph>(SoundSubmix->SoundSubmixGraph)->AddNewSoundSubmix(FromPin, NewSoundSubmix, Location.X, Location.Y);

		AddEditingObject(NewSoundSubmix);
		NewSoundSubmix->PostEditChange();
		NewSoundSubmix->MarkPackageDirty();
	}
}

UEdGraph* FSoundSubmixEditor::GetGraph()
{
	return GraphEditor->GetCurrentGraph();
}

FText FSoundSubmixEditor::GetToolkitName() const
{
	UObject* EditObject = GetEditingObjects()[0];
	return GetLabelForObject(EditObject);
}

void FSoundSubmixEditor::AddMissingEditableSubmixes()
{
	if (UEdGraph* Graph = GraphEditor->GetCurrentGraph())
	{
		bool bChanged = false;
		if (Graph->Nodes.Num() > GetEditingObjects().Num())
		{
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				USoundSubmixGraphNode* GraphNode = CastChecked<USoundSubmixGraphNode>(Node);
				USoundSubmixBase* UntrackedSubmix = GraphNode->SoundSubmix;
				if (UntrackedSubmix && !GetEditingObjects().Contains(UntrackedSubmix))
				{
					bChanged = true;
					AddEditingObject(UntrackedSubmix);
				}
			}
		}

		if (bChanged)
		{
			GraphEditor->NotifyGraphChanged();
		}
	}
}

void FSoundSubmixEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	// When the property editor is in a transaction, make sure edited objects get recorded.
	if (GUndo)
	{
		TArray<TWeakObjectPtr<UObject>> Edited;
		Edited = DetailsView->GetSelectedObjects();

		for (TWeakObjectPtr<UObject> Obj : Edited)
		{
			if (Obj.IsValid())
			{
				Obj->Modify();
			}
		}
	}

	FNotifyHook::NotifyPreChange(PropertyAboutToChange);
}

void FSoundSubmixEditor::SaveAsset_Execute()
{
	TArray<UPackage*> PackagesToSave;

	for (UObject* Obj : GetEditingObjects())
	{
		if (!Obj) continue;
		if (UPackage* Pkg = Obj->GetOutermost())
		{
			if (Pkg->IsDirty())
			{
				PackagesToSave.AddUnique(Pkg);
			}
		}
	}

	// Only attempt to save/checkout dirty ones.
	const bool bCheckDirty = true;
	const bool bPromptToSave = false;
	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
}

void FSoundSubmixEditor::RefreshDetails()
{
	if (DetailsView.IsValid())
	{
		DetailsView->ForceRefresh();
	}
}

void FSoundSubmixEditor::PostUndo(bool bSuccess)
{
	GraphEditor->ClearSelectionSet();
	GraphEditor->NotifyGraphChanged();
}

#undef LOCTEXT_NAMESPACE
