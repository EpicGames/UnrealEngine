// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EditorUndoClient.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "ISoundSubmixEditor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"

class FReferenceCollector;
class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class IDetailsView;
class IToolkitHost;
class SDockTab;
class UEdGraph;
// Forward Declarations
class UEdGraphPin;
class UObject;
class USoundSubmixBase;

class FSoundSubmixEditor : public ISoundSubmixEditor, public FGCObject, public FEditorUndoClient, public FNotifyHook
{
public:
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

	/**
	 * Edits the specified sound submix object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The sound submix to edit
	 */
	void Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit);

	virtual ~FSoundSubmixEditor();

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSoundSubmixEditor");
	}

	/** FAssetEditorToolkit interface */
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void SaveAsset_Execute() override;

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	//~ Begin ISoundSubmixEditor
	void CreateSoundSubmix(UEdGraphPin* FromPin, FVector2D Location, const FString& Name) override;
	//~ End ISoundSubmixEditor

	/** FNotifyHook Interface */
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;

	/** FEditorUndoClient Interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

	void AddMissingEditableSubmixes();

	/** Forces the details panel to re-evaluate property metadata (e.g. EditConditions) after a graph topology change */
	void RefreshDetails();

	/** Select node associated with the provided submix */
	void SelectSubmixes(TSet<USoundSubmixBase*>& InSubmixes);

	/** Returns current graph handled by editor */
	UEdGraph* GetGraph();

private:
	TSharedRef<SDockTab> SpawnTab_GraphCanvas(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets(USoundSubmixBase* InSoundSubmix);

	/** Create new graph editor widget */
	TSharedRef<SGraphEditor> CreateGraphEditorWidget(USoundSubmixBase* InSoundSubmix);

	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<UObject*>& NewSelection);

	/** Called to create context menu when right-clicking on graph */
	FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Adds all children of provided parent submix as editable */
	void AddEditableSubmixChildren(USoundSubmixBase* ParentSubmix);

	/** Select every node in the graph */
	void SelectAllNodes();
	
	/** Whether we can select every node */
	bool CanSelectAllNodes() const;

	/** Remove the currently selected nodes from editor view*/
	void RemoveSelectedNodes();

	/** Whether we are able to remove the currently selected nodes */
	bool CanRemoveNodes() const;

	/** Called to undo the last action */
	void UndoGraphAction();
	
	/** Called to redo the last undone action */
	void RedoGraphAction();

private:
	/** Graph Editor */
	TSharedPtr<SGraphEditor> GraphEditor;

	/** Property View */
	TSharedPtr<IDetailsView> DetailsView;

	/** Command list for this editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	/**	The tab ids for all the tabs used */
	static const FName GraphCanvasTabId;
	static const FName PropertiesTabId;
};