// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixEditorUtilities.h"

#include "EdGraph/EdGraph.h"
#include "ISoundSubmixEditor.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundSubmix.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/ToolkitManager.h"

class IToolkit;

void FSoundSubmixEditorUtilities::CreateSoundSubmix(const UEdGraph* Graph, UEdGraphPin* FromPin, const FVector2D Location, const FString& Name)
{
	check(Graph);

	// Cast outer to SoundSubmix
	USoundSubmixBase* SoundSubmix = CastChecked<USoundSubmixBase>(Graph->GetOuter());

	if (SoundSubmix != nullptr)
	{
		TSharedPtr<ISoundSubmixEditor> SoundSubmixEditor;
		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(SoundSubmix);
		if (FoundAssetEditor.IsValid())
		{
			SoundSubmixEditor = StaticCastSharedPtr<ISoundSubmixEditor>(FoundAssetEditor);
			SoundSubmixEditor->CreateSoundSubmix(FromPin, Location, Name);
		}
	}
}