// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundSubmixGraph/SoundSubmixGraph.h"

#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Sound/SoundSubmix.h"
#include "SoundSubmixGraph/SoundSubmixGraphNode.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "TimerManager.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmixGraph)

class UObject;


USoundSubmixGraph::USoundSubmixGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundSubmixGraph::SetRootSoundSubmix(USoundSubmixBase* InSoundSubmix)
{
	if (RootSoundSubmix)
	{
		// Defer request to close stale editor(s) to avoid property assignments being
		// added to undo/redo transaction stack from the process of reopening the new editor.
		// This can occur if client code calling this function is performed within a scoped
		// transaction (as it most likely is).
		const int32 Size = StaleRoots.Num();
		const int32 Index = StaleRoots.AddUnique(RootSoundSubmix);
		if (Size == Index && GEditor)
		{
			GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([this]()
			{
				if(StaleRoots.Num() > 0)
				{
					check(GEditor);
					if (UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
					{
						for (USoundSubmixBase* Submix : StaleRoots)
						{
							TArray<IAssetEditorInstance*> SubmixEditors = EditorSubsystem->FindEditorsForAsset(Submix);
							for (IAssetEditorInstance* Editor : SubmixEditors)
							{
								Editor->CloseWindow(EAssetEditorCloseReason::EditorRefreshRequested);
							}
						}

						if (RootSoundSubmix)
						{
							if (EditorSubsystem->FindEditorsForAsset(RootSoundSubmix).Num() == 0)
							{
								EditorSubsystem->OpenEditorForAsset(RootSoundSubmix);
							}
						}
					}

					StaleRoots.Reset();
				}
			}));
		}
	}

	RootSoundSubmix = InSoundSubmix;
}

USoundSubmixBase* USoundSubmixGraph::GetRootSoundSubmix() const
{
	return RootSoundSubmix;
}

void USoundSubmixGraph::RebuildGraph()
{
	check(RootSoundSubmix);

	// Don't allow initial graph rebuild to affect package dirty state; remember current state...
	UPackage* Package = GetOutermost();
	const bool bIsDirty = Package->IsDirty();

	Modify();

	RemoveAllNodes();

	ConstructNodes(RootSoundSubmix, 0, 0);

	NotifyGraphChanged();

	// ...and restore it
	Package->SetDirtyFlag(bIsDirty);
}

void USoundSubmixGraph::AddDroppedSoundSubmixes(const TSet<USoundSubmixBase*>& SoundSubmixes, int32 NodePosX, int32 NodePosY)
{
	Modify();

	for (USoundSubmixBase* SoundSubmix : SoundSubmixes)
	{
		NodePosY += ConstructNodes(SoundSubmix, NodePosX, NodePosY);
	}

	NotifyGraphChanged();
}

void USoundSubmixGraph::AddNewSoundSubmix(UEdGraphPin* FromPin, USoundSubmixBase* SoundSubmix, int32 NodePosX, int32 NodePosY, bool bSelectNewNode/* = true*/)
{
	check(SoundSubmix->ChildSubmixes.Num() == 0);

	Modify();

	USoundSubmixGraphNode* GraphNode = CreateNode(SoundSubmix, NodePosX, NodePosY, bSelectNewNode);
	GraphNode->AutowireNewNode(FromPin);

	NotifyGraphChanged();
}

bool USoundSubmixGraph::IsSubmixDisplayed(USoundSubmixBase* SoundSubmix) const
{
	return FindExistingNode(SoundSubmix) != nullptr;
}

void USoundSubmixGraph::LinkSoundSubmixes()
{
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		USoundSubmixGraphNode* Node = CastChecked<USoundSubmixGraphNode>(Nodes[NodeIndex]);

		if (!Node->CheckRepresentsSoundSubmix())
		{
			Node->SoundSubmix->Modify();

			// remove parents of existing children
			for (USoundSubmixBase* ChildSubmix : Node->SoundSubmix->ChildSubmixes)
			{
				if (ChildSubmix)
				{
					ChildSubmix->Modify();
					if (USoundSubmixWithParentBase* SubmixWithParent = CastChecked<USoundSubmixWithParentBase>(ChildSubmix))
					{
						SubmixWithParent->ParentSubmix = nullptr;
					}
				}
			}

			Node->SoundSubmix->ChildSubmixes.Empty();

			if (UEdGraphPin* ChildPin = Node->GetChildPin())
			{
				for (UEdGraphPin* GraphPin : ChildPin->LinkedTo)
				{
					

					if (!GraphPin)
					{
						continue;
					}

					USoundSubmixGraphNode* ChildNode = CastChecked<USoundSubmixGraphNode>(GraphPin->GetOwningNode());

					// If the child submix we're connecting to isn't the type of submix that has an output, continue.
					USoundSubmixWithParentBase* ChildSubmixWithParent = Cast<USoundSubmixWithParentBase>(ChildNode->SoundSubmix);

					if (ChildSubmixWithParent)
					{
						Node->SoundSubmix->ChildSubmixes.Add(ChildNode->SoundSubmix);
						ChildSubmixWithParent->SetParentSubmix(Node->SoundSubmix);
					}
				}
			}

			Node->SoundSubmix->PostEditChange();
			Node->SoundSubmix->MarkPackageDirty();
		}
	}
}

void USoundSubmixGraph::RefreshGraphLinks()
{
	Modify();

	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		USoundSubmixGraphNode* Node = CastChecked<USoundSubmixGraphNode>(Nodes[NodeIndex]);

		if (!Node->CheckRepresentsSoundSubmix())
		{
			UEdGraphPin* ChildPin = Node->GetChildPin();

			Node->Modify();

			ChildPin->BreakAllPinLinks();

			if (Node->SoundSubmix)
			{
				for (int32 ChildIndex = 0; ChildIndex < Node->SoundSubmix->ChildSubmixes.Num(); ChildIndex++)
				{
					USoundSubmixBase* ChildSubmix = Node->SoundSubmix->ChildSubmixes[ChildIndex];

					if (ChildSubmix)
					{
						USoundSubmixGraphNode* ChildNode = FindExistingNode(ChildSubmix);

						if (!ChildNode)
						{
							// New Child not yet represented on graph
							ConstructNodes(ChildSubmix, Node->NodePosX + 400, Node->NodePosY);
							ChildNode = FindExistingNode(ChildSubmix);
						}

						ChildPin->MakeLinkTo(ChildNode->GetParentPin());
					}
				}
			}

			Node->PostEditChange();
		}
	}

	NotifyGraphChanged();
}

void USoundSubmixGraph::RecursivelyRemoveNodes(const TSet<UObject*> NodesToRemove)
{
	Modify();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(NodesToRemove); NodeIt; ++NodeIt)
	{
		USoundSubmixGraphNode* Node = Cast<USoundSubmixGraphNode>(*NodeIt);

		if (Node && Node->CanUserDeleteNode())
		{
			RecursivelyRemoveNode(Node);
		}
	}

	LinkSoundSubmixes();
}

int32 USoundSubmixGraph::ConstructNodes(USoundSubmixBase* SoundSubmix, int32 NodePosX, int32 NodePosY, bool bSelectNewNode/* = true*/)
{
	check(SoundSubmix);

	TMap<USoundSubmixBase*, int32> ChildCounts;

	RecursivelyGatherChildCounts(SoundSubmix, ChildCounts);

	USoundSubmixGraphNode* GraphNode = CreateNode(SoundSubmix, NodePosX, NodePosY, bSelectNewNode);

	return RecursivelyConstructChildNodes(GraphNode, ChildCounts);
}

int32 USoundSubmixGraph::RecursivelyGatherChildCounts(USoundSubmixBase* ParentSubmix, TMap<USoundSubmixBase*, int32>& OutChildCounts)
{
	int32 ChildSize = 0;

	for (int32 ChildIndex = 0; ChildIndex < ParentSubmix->ChildSubmixes.Num(); ChildIndex++)
	{
		if (ParentSubmix->ChildSubmixes[ChildIndex])
		{
			ChildSize += RecursivelyGatherChildCounts(ParentSubmix->ChildSubmixes[ChildIndex], OutChildCounts);
		}
	}

	if (ChildSize == 0)
	{
		ChildSize = 1;
	}

	OutChildCounts.Add(ParentSubmix, ChildSize);
	return ChildSize;
}

int32 USoundSubmixGraph::RecursivelyConstructChildNodes(USoundSubmixGraphNode* ParentNode, const TMap<USoundSubmixBase*, int32>& InChildCounts, bool bSelectNewNode /* = true*/)
{
	static const int32 HorizontalSpacing = -400;
	static const int32 VerticalSpacing = 100;

	USoundSubmixBase* ParentSubmix = ParentNode->SoundSubmix;
	int32 TotalChildSizeY = InChildCounts.FindChecked(ParentSubmix) * VerticalSpacing;
	int32 NodeStartY = ParentNode->NodePosY - (TotalChildSizeY * 0.5f) + (VerticalSpacing * 0.5f);
	int32 NodePosX = ParentNode->NodePosX + HorizontalSpacing;

	for (int32 ChildIndex = 0; ChildIndex < ParentSubmix->ChildSubmixes.Num(); ChildIndex++)
	{
		if (ParentSubmix->ChildSubmixes[ChildIndex])
		{
			const int32 ChildCount = InChildCounts.FindChecked(ParentSubmix->ChildSubmixes[ChildIndex]);
			int32 NodePosY = NodeStartY + (ChildCount * VerticalSpacing * 0.5f) - (VerticalSpacing * 0.5f);
			USoundSubmixGraphNode* ChildNode = CreateNode(ParentSubmix->ChildSubmixes[ChildIndex], NodePosX, NodePosY, bSelectNewNode);
			ParentNode->GetChildPin()->MakeLinkTo(ChildNode->GetParentPin());
			RecursivelyConstructChildNodes(ChildNode, InChildCounts);
			NodeStartY += ChildCount * VerticalSpacing;
		}
	}

	return TotalChildSizeY;
}

void USoundSubmixGraph::RecursivelyRemoveNode(USoundSubmixGraphNode* ParentNode)
{
	UEdGraphPin* ChildPin = ParentNode->GetChildPin();

	for (int32 ChildIndex = ChildPin->LinkedTo.Num() - 1; ChildIndex >= 0; ChildIndex--)
	{
		USoundSubmixGraphNode* ChildNode = CastChecked<USoundSubmixGraphNode>(ChildPin->LinkedTo[ChildIndex]->GetOwningNode());
		RecursivelyRemoveNode(ChildNode);
	}

	ParentNode->Modify();
	RemoveNode(ParentNode);
}

void USoundSubmixGraph::RemoveAllNodes()
{
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		NodesToRemove[NodeIndex]->Modify();
		RemoveNode(NodesToRemove[NodeIndex]);
	}
}

USoundSubmixGraphNode* USoundSubmixGraph::CreateNode(USoundSubmixBase* SoundSubmix, int32 NodePosX, int32 NodePosY, bool bSelectNewNode/* = true*/)
{
	USoundSubmixGraphNode* GraphNode = FindExistingNode(SoundSubmix);

	if (!GraphNode)
	{
		FGraphNodeCreator<USoundSubmixGraphNode> NodeCreator(*this);
		GraphNode = NodeCreator.CreateNode(bSelectNewNode);
		GraphNode->SoundSubmix = SoundSubmix;
		GraphNode->NodePosX = NodePosX;
		GraphNode->NodePosY = NodePosY;
		NodeCreator.Finalize();
	}
	return GraphNode;
}

USoundSubmixGraphNode* USoundSubmixGraph::FindExistingNode(USoundSubmixBase* SoundSubmix) const
{
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
	{
		USoundSubmixGraphNode* Node = CastChecked<USoundSubmixGraphNode>(Nodes[NodeIndex]);
		if (Node->SoundSubmix == SoundSubmix)
		{
			return Node;
		}
	}

	return nullptr;
}
