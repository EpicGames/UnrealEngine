// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixGraph/SoundSubmixGraphNode.h"

#include "Audio/AudioWidgetSubsystem.h"
#include "Audio/SoundSubmixWidgetInterface.h"
#include "Blueprint/UserWidget.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Sound/SoundSubmix.h"
#include "SoundSubmixDefaultColorPalette.h"
#include "SoundSubmixEditor.h"
#include "SoundSubmixGraph/SoundSubmixGraph.h"
#include "SoundSubmixGraph/SoundSubmixGraphSchema.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmixGraphNode)

class SWidget;
class UWorld;


#define LOCTEXT_NAMESPACE "SoundSubmixGraphNode"


void SSubmixGraphNode::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	SubmixBase = InArgs._SubmixBase;
	SubmixNodeUserWidget = InArgs._SubmixNodeUserWidget;

	if (SubmixNodeUserWidget.IsValid())
	{
		ISoundSubmixWidgetInterface::Execute_OnConstructed(SubmixNodeUserWidget.Get(), SubmixBase.Get());
	}
	GraphNode = InGraphNode;
	UpdateGraphNode();
}

TSharedRef<SWidget> SSubmixGraphNode::CreateNodeContentArea()
{	
	if (SubmixNodeUserWidget.IsValid())
	{
		// NODE CONTENT AREA
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0, 3))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.0f)
					[
						// LEFT
						SAssignNew(LeftNodeBox, SVerticalBox)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						// RIGHT
						SAssignNew(RightNodeBox, SVerticalBox)
					]

				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.0f)
					[
						SubmixNodeUserWidget->TakeWidget()
					]
				]
			]
		;
	}
	else
	{
		return SGraphNode::CreateNodeContentArea();
	}

}



USoundSubmixGraphNode::USoundSubmixGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ChildPin(NULL)
	, ParentPin(NULL)
{
}

bool USoundSubmixGraphNode::CheckRepresentsSoundSubmix()
{
	if (!SoundSubmix)
	{
		return false;
	}

	for (int32 ChildIndex = 0; ChildIndex < ChildPin->LinkedTo.Num(); ChildIndex++)
	{
		USoundSubmixGraphNode* ChildNode = CastChecked<USoundSubmixGraphNode>(ChildPin->LinkedTo[ChildIndex]->GetOwningNode());
		if (!SoundSubmix->ChildSubmixes.Contains(ChildNode->SoundSubmix))
		{
			return false;
		}
	}

	for (int32 ChildIndex = 0; ChildIndex < SoundSubmix->ChildSubmixes.Num(); ChildIndex++)
	{
		bool bFoundChild = false;
		for (int32 NodeChildIndex = 0; NodeChildIndex < ChildPin->LinkedTo.Num(); NodeChildIndex++)
		{
			USoundSubmixGraphNode* ChildNode = CastChecked<USoundSubmixGraphNode>(ChildPin->LinkedTo[NodeChildIndex]->GetOwningNode());
			if (ChildNode->SoundSubmix == SoundSubmix->ChildSubmixes[ChildIndex])
			{
				bFoundChild = true;
				break;
			}
		}

		if (!bFoundChild)
		{
			return false;
		}
	}

	return true;
}

FLinearColor USoundSubmixGraphNode::GetNodeTitleColor() const
{
	return Audio::GetColorForSubmixType(SoundSubmix);
}

void USoundSubmixGraphNode::AllocateDefaultPins()
{
	check(Pins.Num() == 0);

	ChildPin = CreatePin(EGPD_Input, Audio::GetNameForSubmixType(SoundSubmix), *LOCTEXT("SoundSubmixGraphNode_Input", "Input").ToString());

	if (USoundSubmixWithParentBase* NonEndpointSubmix = Cast<USoundSubmixWithParentBase>(SoundSubmix))
	{
		ParentPin = CreatePin(EGPD_Output, Audio::GetNameForSubmixType(SoundSubmix), *LOCTEXT("SoundSubmixGraphNode_Output", "Output").ToString());
	}
}

void USoundSubmixGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin)
	{
		const USoundSubmixGraphSchema* Schema = CastChecked<USoundSubmixGraphSchema>(GetSchema());

		if (FromPin->Direction == EGPD_Input)
		{
			Schema->TryCreateConnection(FromPin, ParentPin);
		}
		else
		{
			Schema->TryCreateConnection(FromPin, ChildPin);
		}
	}
}

bool USoundSubmixGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(USoundSubmixGraphSchema::StaticClass());
}

bool USoundSubmixGraphNode::CanUserDeleteNode() const
{
	check(GEditor);
	UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<IAssetEditorInstance*> SubmixEditors = EditorSubsystem->FindEditorsForAsset(SoundSubmix);
	for (IAssetEditorInstance* Editor : SubmixEditors)
	{
		if (!Editor)
		{
			continue;
		}

		FSoundSubmixEditor* SubmixEditor = static_cast<FSoundSubmixEditor*>(Editor);
		if (UEdGraph* Graph = SubmixEditor->GetGraph())
		{
			if (SoundSubmix->SoundSubmixGraph == Graph)
			{
				USoundSubmixBase* RootSubmix = CastChecked<USoundSubmixGraph>(Graph)->GetRootSoundSubmix();
				if (RootSubmix == SoundSubmix)
				{
					return false;
				}
			}
		}
	}

	return UEdGraphNode::CanUserDeleteNode();
}

FText USoundSubmixGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (SoundSubmix)
	{
		return FText::FromString(SoundSubmix->GetName());
	}
	else
	{
		return Super::GetNodeTitle(TitleType);
	}
}

TSharedPtr<SGraphNode> USoundSubmixGraphNode::CreateVisualWidget()
{
	if (UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr)
	{
		if (USoundSubmixBase* SubmixBase = Cast<USoundSubmixBase>(SoundSubmix))
		{
			if (UAudioWidgetSubsystem* AudioWidgetSubsystem = GEngine ? GEngine->GetEngineSubsystem<UAudioWidgetSubsystem>() : nullptr)
			{
				TArray<UUserWidget*> UserWidgets = AudioWidgetSubsystem->CreateUserWidgets(*World, USoundSubmixWidgetInterface::StaticClass());
				if (!UserWidgets.IsEmpty())
				{
					// For now, only supports single widget. Gallery system to be implemented to support
					// showing multiple widgets and/or cycling node widgets.
					SubmixNodeUserWidget = UserWidgets[0];
				}
			}

			// Pass the owning submix and the user widgets to the graph node
			return SNew(SSubmixGraphNode, this)
				.SubmixBase(SubmixBase)
				.SubmixNodeUserWidget(SubmixNodeUserWidget);
		}
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
