// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixGraph/SoundSubmixGraphSchema.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "ISoundfieldEndpoint.h"
#include "ISoundfieldFormat.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundSubmix.h"
#include "SoundSubmixDefaultColorPalette.h"
#include "SoundSubmixEditor.h"
#include "SoundSubmixEditorUtilities.h"
#include "SoundSubmixGraph/SoundSubmixGraph.h"
#include "SoundSubmixGraph/SoundSubmixGraphNode.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmixGraphSchema)

class FSlateRect;
class FSlateWindowElementList;

#define LOCTEXT_NAMESPACE "SoundSubmixSchema"

FConnectionDrawingPolicy* FSoundSubmixGraphConnectionDrawingPolicyFactory::CreateConnectionPolicy(
	const UEdGraphSchema* Schema,
	int32 InBackLayerID,
	int32 InFrontLayerID,
	float ZoomFactor,
	const FSlateRect& InClippingRect,
	FSlateWindowElementList& InDrawElements,
	UEdGraph* InGraphObj) const
{
	if (Schema->IsA(USoundSubmixGraphSchema::StaticClass()))
	{
		return new FSoundSubmixGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
	}
	return nullptr;
}

FSoundSubmixGraphConnectionDrawingPolicy::FSoundSubmixGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(InGraphObj)
{
	ActiveWireThickness = Settings->TraceAttackWireThickness;
	InactiveWireThickness = Settings->TraceReleaseWireThickness;
}

// Give specific editor modes a chance to highlight this connection or darken non-interesting connections
void FSoundSubmixGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FConnectionParams& OutParams)
{
	if (!(OutputPin && InputPin && GraphObj))
	{
		return;
	}

	OutParams.AssociatedPin1 = InputPin;
	OutParams.AssociatedPin2 = OutputPin;

	// Get the schema and grab the default color from it
	const UEdGraphSchema* Schema = GraphObj->GetSchema();

	OutParams.WireColor = Schema->GetPinTypeColor(OutputPin->PinType);

	bool bExecuted = false;

	USoundSubmixBase* InputSubmix = OutputPin ? CastChecked<USoundSubmixGraphNode>(OutputPin->GetOwningNode())->SoundSubmix : nullptr;
	USoundSubmixBase* OutputSubmix = InputPin ? CastChecked<USoundSubmixGraphNode>(InputPin->GetOwningNode())->SoundSubmix : nullptr;

	// Run through the predecessors, and on
	if (FExecPairingMap* PredecessorMap = PredecessorNodes.Find(OutputPin->GetOwningNode()))
	{
		if (FTimePair* Times = PredecessorMap->Find(InputPin->GetOwningNode()))
		{
			bExecuted = true;

			OutParams.WireThickness = ActiveWireThickness;
			OutParams.WireColor = Audio::GetColorForSubmixType(OutputSubmix);
			OutParams.bDrawBubbles = Audio::IsConnectionPerformingSoundfieldConversion(InputSubmix, OutputSubmix);
		}
	}

	if (!bExecuted)
	{
		OutParams.WireColor = Audio::GetColorForSubmixType(InputSubmix);
		OutParams.WireThickness = InactiveWireThickness;
	}
}

UEdGraphNode* FSoundSubmixGraphSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	FSoundSubmixEditorUtilities::CreateSoundSubmix(ParentGraph, FromPin, FDeprecateSlateVector2D(Location), NewSoundSubmixName);
	return nullptr;
}

USoundSubmixGraphSchema::USoundSubmixGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USoundSubmixGraphSchema::ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	USoundSubmixGraphNode* InputNode = CastChecked<USoundSubmixGraphNode>(InputPin->GetOwningNode());
	USoundSubmixGraphNode* OutputNode = CastChecked<USoundSubmixGraphNode>(OutputPin->GetOwningNode());

	// Master Submix cannot be an input as it would create an inferred loop for submixes without an explicit parent
	if (const UAudioSettings* Settings = GetDefault<UAudioSettings>())
	{
		if (USoundSubmix* MasterSubmix = Cast<USoundSubmix>(Settings->MasterSubmix.TryLoad()))
		{
			if (OutputNode->SoundSubmix == MasterSubmix)
			{
				return true;
			}

			if (SubmixUtils::FindInGraph(MasterSubmix, OutputNode->SoundSubmix, false))
			{
				return true;
			}
		}
	}

	return SubmixUtils::FindInGraph(OutputNode->SoundSubmix, InputNode->SoundSubmix, false);
}

void USoundSubmixGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutOkIcon = true;
	OutTooltipText = LOCTEXT("DropSoundSubmix", "Add Submix to Graph.").ToString();

	if (HoverGraph)
	{
		TArray<USoundSubmixGraphNode*> SubmixNodes;
		HoverGraph->GetNodesOfClass<USoundSubmixGraphNode>(SubmixNodes);
		for (USoundSubmixGraphNode* SubmixNode : SubmixNodes)
		{
			if (SubmixNode && SubmixNode->SoundSubmix)
			{
				auto MatchesSubmix = [&](const FAssetData& Asset)
				{
					return Asset.GetFullName() == SubmixNode->SoundSubmix->GetFullName();
				};

				if (Assets.ContainsByPredicate(MatchesSubmix))
				{
					OutOkIcon = false;
					OutTooltipText = LOCTEXT("SoundSubmixInGraph", "Selected asset or assets already in graph.").ToString();
					break;
				}
			}
		}
	}

	for (const FAssetData& Data : Assets)
	{
		if (!Data.IsInstanceOf(USoundSubmixBase::StaticClass()))
		{
			OutOkIcon = false;
			OutTooltipText = LOCTEXT("NotASoundSubmix", "Asset(s) must all be Submixes.").ToString();
			break;
		}
	}
}

void USoundSubmixGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const FText Name = LOCTEXT("NewSoundSubmix", "New Sound Submix");
	const FText ToolTip = LOCTEXT("NewSoundSubmixTooltip", "Create a new sound submix");
	
	TSharedPtr<FSoundSubmixGraphSchemaAction_NewNode> NewAction(new FSoundSubmixGraphSchemaAction_NewNode(FText::GetEmpty(), Name, ToolTip, 0));

	ContextMenuBuilder.AddAction(NewAction);
}

void USoundSubmixGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node)
	{
		const USoundSubmixGraphNode* SoundGraphNode = Cast<const USoundSubmixGraphNode>(Context->Node);
		{
			FToolMenuSection& Section = Menu->AddSection("SoundSubmixGraphSchemaNodeActions", LOCTEXT("ClassActionsMenuHeader", "SoundSubmix Actions"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
			Section.AddMenuEntry(FGenericCommands::Get().Delete);
		}
	}

	// No Super call so Node comments option is not shown
}

const FPinConnectionResponse USoundSubmixGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatible", "Directions are not compatible"));
	}

	// Note- are input pin and output pin swapped here? Am I losing it?
	USoundSubmixBase* InputSubmix = CastChecked<USoundSubmixGraphNode>(OutputPin->GetOwningNode())->SoundSubmix;
	USoundSubmixBase* OutputSubmix = CastChecked<USoundSubmixGraphNode>(InputPin->GetOwningNode())->SoundSubmix;

	// Forbid connecting dynamic submixes to other submixes.
	if (InputSubmix->IsDynamic( false /*bIncludeAncestors*/ ))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("SubmixIsDynamic", "Submix you are trying to connect from is dynamic and shouldn't have any static parents"));
	}

	// Check to see if this is an endpoint submix.
	if (!InputSubmix->IsA<USoundSubmixWithParentBase>())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("SubmixIsEndpoint", "Submix you are trying to connect from is an endpoint."));
	}

	// If we're trying to make a connection between two soundfield submixes, ensure that we can transcode between the two.
	if (InputSubmix->IsA<USoundfieldSubmix>() && (OutputSubmix->IsA<USoundfieldSubmix>() || OutputSubmix->IsA<USoundfieldEndpointSubmix>()))
	{
		USoundfieldSubmix* InputSoundfieldSubmix = Cast<USoundfieldSubmix>(InputSubmix);
		USoundfieldSubmix* OutputSoundfieldSubmix = Cast<USoundfieldSubmix>(OutputSubmix);

		ISoundfieldFactory* InputFactory = InputSoundfieldSubmix->GetSoundfieldFactoryForSubmix();
		ISoundfieldFactory* OutputFactory = nullptr;

		const USoundfieldEncodingSettingsBase* InputEncodingSettings = InputSoundfieldSubmix->GetEncodingSettings();
		const USoundfieldEncodingSettingsBase* OutputEncodingSettings = nullptr;

		if (!OutputSoundfieldSubmix)
		{
			USoundfieldEndpointSubmix* SoundfieldEndpointSubmixB = CastChecked<USoundfieldEndpointSubmix>(OutputSubmix);
			OutputFactory = SoundfieldEndpointSubmixB->GetSoundfieldEndpointForSubmix();
			OutputEncodingSettings = SoundfieldEndpointSubmixB->GetEncodingSettings();
		}
		else
		{
			OutputFactory = OutputSoundfieldSubmix->GetSoundfieldFactoryForSubmix();
			OutputEncodingSettings = OutputSoundfieldSubmix->GetEncodingSettings();
		}

		if (InputFactory && OutputFactory)
		{
			if (!InputEncodingSettings)
			{
				InputEncodingSettings = InputFactory->GetDefaultEncodingSettings();
			}

			if (!InputEncodingSettings)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("SoundfieldSubmixSourceIsInvalid", "Submix you are trying to connect from does not specify default settings. Please implement ISoundfieldFactory::GetDefaultEncodingSettings."));
			}

			TUniquePtr<ISoundfieldEncodingSettingsProxy> InputEncodingSettingsProxy = InputEncodingSettings->GetProxy();

			if (!InputEncodingSettingsProxy)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("SourceSoundfieldEncodingSettingsAreInvalid", "Submix you are trying to connect from failed to generate a proxy of it's settings. Please check USoundfieldEncodingSettingsBase::GetProxy()."));
			}


			if (!OutputEncodingSettings)
			{
				OutputEncodingSettings = OutputFactory->GetDefaultEncodingSettings();
			}

			if (!OutputEncodingSettings)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("SoundfieldSubmixDestIsInvalid", "Submix you are trying to connect to does not specify default settings. Please implement ISoundfieldFactory::GetDefaultEncodingSettings."));
			}

			TUniquePtr<ISoundfieldEncodingSettingsProxy> OutputEncodingSettingsProxy = OutputEncodingSettings->GetProxy();

			if (!OutputEncodingSettingsProxy)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DestSoundfieldEncodingSettingsAreInvalid", "Submix you are trying to connect to failed to generate a proxy of it's settings. Please check USoundfieldEncodingSettingsBase::GetProxy()."));
			}

			const bool bAreSoundfieldsCompatible = InputFactory->CanTranscodeToSoundfieldFormat(OutputFactory->GetSoundfieldFormatName(), *OutputEncodingSettingsProxy) || OutputFactory->CanTranscodeFromSoundfieldFormat(InputFactory->GetSoundfieldFormatName(), *InputEncodingSettingsProxy);
			if (!bAreSoundfieldsCompatible)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("SoundfieldSubmixesAreIncompatible", "These two submixes have incompatible types."));
			}
		}
	}

	if (ConnectionCausesLoop(InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop"));
	}

	// Break existing connections on outputs only - multiple input connections are acceptable
	if (OutputPin->LinkedTo.Num() > 0)
	{
		ECanCreateConnectionResponse ReplyBreakInputs;
		if (OutputPin == PinA)
		{
			ReplyBreakInputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
		}
		else
		{
			ReplyBreakInputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
		}
		return FPinConnectionResponse(ReplyBreakInputs, LOCTEXT("ConnectionReplace", "Replace existing connections"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, FText::GetEmpty());
}

bool USoundSubmixGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	check(PinA);
	check(PinB);

	bool bModified = UEdGraphSchema::TryCreateConnection(PinA, PinB);

	if (bModified)
	{
		USoundSubmixGraph* Graph = CastChecked<USoundSubmixGraph>(PinA->GetOwningNode()->GetGraph());
		Graph->LinkSoundSubmixes();

		USoundSubmixBase* SubmixA = CastChecked<USoundSubmixGraphNode>(PinA->GetOwningNode())->SoundSubmix;
		USoundSubmixBase* SubmixB = CastChecked<USoundSubmixGraphNode>(PinB->GetOwningNode())->SoundSubmix;

		USoundSubmixWithParentBase* SubmixWithParentA = Cast<USoundSubmixWithParentBase>(SubmixA);
		USoundSubmixWithParentBase* SubmixWithParentB = Cast<USoundSubmixWithParentBase>(SubmixA);

		// If re-basing root, re-open editor.  This will force the root to be the primary edited node
		if (Graph->GetRootSoundSubmix() == SubmixA && SubmixWithParentA && SubmixWithParentA->ParentSubmix != nullptr)
		{
			Graph->SetRootSoundSubmix(SubmixWithParentA->ParentSubmix);
		}
		else if (Graph->GetRootSoundSubmix() == SubmixB && SubmixWithParentB && SubmixWithParentB->ParentSubmix != nullptr)
		{
			Graph->SetRootSoundSubmix(SubmixWithParentB->ParentSubmix);
		}
	}

	return bModified;
}

bool USoundSubmixGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	return true;
}

FLinearColor USoundSubmixGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return Audio::GetColorForSubmixType(PinType.PinCategory);
}

void USoundSubmixGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	Super::BreakNodeLinks(TargetNode);

	CastChecked<USoundSubmixGraph>(TargetNode.GetGraph())->LinkSoundSubmixes();
}

void USoundSubmixGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links") );

	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);
	
	// if this would notify the node then we need to re-link sound classes
	if (bSendsNodeNotifcation)
	{
		if (USoundSubmixGraphNode* GraphNode = Cast<USoundSubmixGraphNode>(TargetPin.GetOwningNode()))
		{
			// If TargetPin is an input, We should break links to all child submixes of the submix that owns this pin.
			if (TargetPin.Direction == EEdGraphPinDirection::EGPD_Input)
			{
				// Iterate through all child submixes
				USoundSubmixBase* OutputSubmix = GraphNode->SoundSubmix;

				// Note: If we ever support multiple parents for submixes, this will need to be modified.
				for (USoundSubmixBase* InputSubmix : OutputSubmix->ChildSubmixes)
				{
					if (USoundSubmixWithParentBase* SubmixWithParent = Cast<USoundSubmixWithParentBase>(InputSubmix))
					{
						SubmixWithParent->ParentSubmix = nullptr;
						SubmixWithParent->PostEditChange();
					}
				}

				OutputSubmix->ChildSubmixes.Reset();
				OutputSubmix->PostEditChange();
			}
			else if (TargetPin.Direction == EEdGraphPinDirection::EGPD_Output)
			{
				// If this is an output pin, break the connection between this submix and it's parent.
				USoundSubmixWithParentBase* InputSubmix = CastChecked<USoundSubmixWithParentBase>(GraphNode->SoundSubmix);
				USoundSubmixBase* OutputSubmix = InputSubmix->ParentSubmix;
				check(OutputSubmix);

				OutputSubmix->ChildSubmixes.Remove(InputSubmix);
				InputSubmix->ParentSubmix = nullptr;

				OutputSubmix->PostEditChange();
				InputSubmix->PostEditChange();
			}
		}

		CastChecked<USoundSubmixGraph>(TargetPin.GetOwningNode()->GetGraph())->LinkSoundSubmixes();
	}
}

void USoundSubmixGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link") );
	Super::BreakSinglePinLink(SourcePin, TargetPin);

	// Compare the directions
	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(SourcePin, TargetPin, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return;
	}

	// Note- are input pin and output pin swapped here? Am I losing it?
	USoundSubmixBase* InputSubmix = CastChecked<USoundSubmixGraphNode>(OutputPin->GetOwningNode())->SoundSubmix;
	USoundSubmixBase* OutputSubmix = CastChecked<USoundSubmixGraphNode>(InputPin->GetOwningNode())->SoundSubmix;

	if (USoundSubmixWithParentBase* SubmixWithParent = CastChecked<USoundSubmixWithParentBase>(InputSubmix))
	{
		SubmixWithParent->ParentSubmix = nullptr;
		SubmixWithParent->PostEditChange();
	}

	CastChecked<USoundSubmixGraph>(SourcePin->GetOwningNode()->GetGraph())->LinkSoundSubmixes();
}

void USoundSubmixGraphSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	check(GEditor);
	check(Graph);

	USoundSubmixGraph* SoundSubmixGraph = CastChecked<USoundSubmixGraph>(Graph);
	TSet<IAssetEditorInstance*> Editors;
	TSet<USoundSubmixBase*> UndisplayedSubmixes;
	for (const FAssetData& Asset : Assets)
	{
		if (USoundSubmixBase* SoundSubmix = Cast<USoundSubmixBase>(Asset.GetAsset()))
		{
			// Walk to the root submix
			USoundSubmixWithParentBase* SubmixWithParent = Cast<USoundSubmixWithParentBase>(SoundSubmix);
			while (SubmixWithParent && SubmixWithParent->ParentSubmix != nullptr)
			{
				SoundSubmix = SubmixWithParent->ParentSubmix;
				SubmixWithParent = Cast<USoundSubmixWithParentBase>(SoundSubmix);
			}

			if (!SoundSubmixGraph->IsSubmixDisplayed(SoundSubmix))
			{
				UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				TArray<IAssetEditorInstance*> SubmixEditors = EditorSubsystem->FindEditorsForAsset(SoundSubmix);
				for (IAssetEditorInstance* Editor : SubmixEditors)
				{
					if (Editor)
					{
						Editors.Add(Editor);
					}
				}
				UndisplayedSubmixes.Add(SoundSubmix);
			}
		}
	}

	if (UndisplayedSubmixes.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("SoundSubmixEditorDropSubmixes", "Sound Submix Editor: Drag and Drop Sound Submix"));

		for (IAssetEditorInstance* Editor : Editors)
		{
			check(Editor);
			FSoundSubmixEditor* SubmixEditor = static_cast<FSoundSubmixEditor*>(Editor);

			// Close editors with dropped (and undisplayed) submix branches as they are now displayed locally in this graph
			// (to avoid modification of multiple graph editors representing the same branch of submixes)
			if (SubmixEditor->GetGraph() != Graph)
			{
				Editor->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
			}
		}

		// If editor is this graph's editor, update editable objects and select dropped submixes.
		if (USoundSubmixBase* RootSubmix = SoundSubmixGraph->GetRootSoundSubmix())
		{
			UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			if (IAssetEditorInstance* EditorInstance = EditorSubsystem->FindEditorForAsset(RootSubmix, false /* bFocusIfOpen */))
			{
				FSoundSubmixEditor* SubmixEditor = static_cast<FSoundSubmixEditor*>(EditorInstance);
				SoundSubmixGraph->AddDroppedSoundSubmixes(UndisplayedSubmixes, GraphPosition.X, GraphPosition.Y);
				SubmixEditor->AddMissingEditableSubmixes();
				SubmixEditor->SelectSubmixes(UndisplayedSubmixes);
			}
		}
	}
}
#undef LOCTEXT_NAMESPACE
