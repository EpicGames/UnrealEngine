// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubmixDetailsCustomization.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "IAudioEndpoint.h"
#include "IDetailPropertyRow.h"
#include "ISoundfieldEndpoint.h"
#include "ISoundfieldFormat.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "Sound/SoundSubmix.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SubmixDetailsInspector"

void FNameSelectorGenerator::SetWeakThis(TWeakPtr<FNameSelectorGenerator>&& InWeakThis)
{
	checkSlow(InWeakThis.IsValid() && !WeakThis.IsValid());
	WeakThis = MoveTemp(InWeakThis);
}

FText FNameSelectorGenerator::GetComboBoxContent() const
{
	FName DesiredName = CachedCallbacks.GetCurrentlySelectedName();
	return FText::FromName(DesiredName);
}


FText FNameSelectorGenerator::GetComboBoxToolTip() const
{
	return FText::FromString(CachedCallbacks.GetTooltipText());
}

TSharedRef<SWidget> FNameSelectorGenerator::MakeNameSelectorWidget(TArray<FName>& InNameArray, FNameSelectorCallbacks&& InCallbacks)
{
	CachedNameArray.Reset();

	for (FName& InName : InNameArray)
	{
		CachedNameArray.Add(TSharedPtr<FName>(new FName(InName)));
	}

	CachedCallbacks = MoveTemp(InCallbacks);

	FName CurrentlySelectedName = CachedCallbacks.GetCurrentlySelectedName();
	TSharedPtr<FName> InitialSelectedName;

	for (TSharedPtr<FName>& NameSelection : CachedNameArray)
	{
		if (*NameSelection == CurrentlySelectedName)
		{
			InitialSelectedName = NameSelection;
			break;
		}
	}

	TSharedRef<FNameSelectorGenerator> SharedThis = WeakThis.Pin().ToSharedRef();

	return SNew(SComboBox<TSharedPtr<FName>>)
		.OnGenerateWidget(SharedThis, &FNameSelectorGenerator::HandleResponseComboBoxGenerateWidget)
		.OptionsSource(&CachedNameArray)
		.OnSelectionChanged(SharedThis, &FNameSelectorGenerator::OnSelectionChanged)
		.InitiallySelectedItem(InitialSelectedName)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		.ContentPadding(FMargin(2.0f, 2.0f))
		.Content()
		[
			SNew(STextBlock)
			.Text(SharedThis, &FNameSelectorGenerator::GetComboBoxContent)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(SharedThis, &FNameSelectorGenerator::GetComboBoxToolTip)
		];
}

void FNameSelectorGenerator::OnSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo)
{
	check(NameItem.IsValid());

	CachedCallbacks.OnNewNameSelected(*NameItem);
}

TSharedRef<SWidget> FNameSelectorGenerator::HandleResponseComboBoxGenerateWidget(TSharedPtr<FName> StringItem)
{
	return SNew(STextBlock)
		.Text(FText::FromName(*StringItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

FNameSelectorGenerator::FNameSelectorGenerator(FProtectedToken)
{
}

TSharedRef<FNameSelectorGenerator> FNameSelectorGenerator::MakeInstance()
{
	TSharedRef<FNameSelectorGenerator> Result = MakeShared<FNameSelectorGenerator>(FProtectedToken{});
	Result->SetWeakThis(Result.ToWeakPtr());
	return Result;
}

FSoundfieldSubmixDetailsCustomization::FSoundfieldSubmixDetailsCustomization(FPrivateToken)
{
}

TSharedRef<IDetailCustomization> FSoundfieldSubmixDetailsCustomization::MakeInstance()
{
	return MakeShared<FSoundfieldSubmixDetailsCustomization>(FPrivateToken{});
}

void FSoundfieldSubmixDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	if (!SoundfieldFormatNameSelectorGenerator)
	{
		SoundfieldFormatNameSelectorGenerator = FNameSelectorGenerator::MakeInstance();
	}

	IDetailCategoryBuilder& SoundfieldCategory = DetailLayout.EditCategory(TEXT("Soundfield"));

	TSharedPtr<IPropertyHandle> EncodingFormatPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USoundfieldSubmix, SoundfieldEncodingFormat));
	IDetailPropertyRow& EncodingFormatPropertyRow = SoundfieldCategory.AddProperty(EncodingFormatPropertyHandle);
	check(EncodingFormatPropertyHandle);

	TWeakPtr<IPropertyHandle> WeakEncodingFormatPtr = EncodingFormatPropertyHandle;

	// Set up callbacks for the name selector dropdown menu:
	FNameSelectorGenerator::FNameSelectorCallbacks Callbacks =
	{
		// void OnNameChanged(FName InName)
		[EncodingFormatPropertyHandle](FName SelectedName)
		{
			if (EncodingFormatPropertyHandle.IsValid() && EncodingFormatPropertyHandle->IsValidHandle())
			{
				EncodingFormatPropertyHandle->SetValue(SelectedName);
			}
		},
		// FName GetCurrentName()
		[EncodingFormatPropertyHandle]()
		{
			FName SelectedName;

			if (EncodingFormatPropertyHandle.IsValid() && EncodingFormatPropertyHandle->IsValidHandle())
			{
				EncodingFormatPropertyHandle->GetValue(SelectedName);
			}

			return SelectedName;
		},
		// FString GetTooltipText()
		[]()
		{
			return FString(TEXT("Use this to select the channel or soundfield configuration of this submix."));
		}
	};

	TArray<FName> AvailableSoundfieldFormats = ISoundfieldFactory::GetAvailableSoundfieldFormats();

	EncodingFormatPropertyRow.CustomWidget()
		.NameContent()
		[
			EncodingFormatPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(1000.0f)
		.MinDesiredWidth(50.0f)
		[
			SoundfieldFormatNameSelectorGenerator->MakeNameSelectorWidget(AvailableSoundfieldFormats, MoveTemp(Callbacks))
		];
}

FEndpointSubmixDetailsCustomization::FEndpointSubmixDetailsCustomization(FProtectedToken ProtectedToken)
	: FNameSelectorGenerator(ProtectedToken)
{
}

TSharedRef<IDetailCustomization> FEndpointSubmixDetailsCustomization::MakeInstance()
{
	TSharedRef<FEndpointSubmixDetailsCustomization> Result = MakeShared<FEndpointSubmixDetailsCustomization>(FProtectedToken{});
	Result->SetWeakThis(StaticCastWeakPtr<FNameSelectorGenerator>(Result.ToWeakPtr()));
	return Result;
}

void FEndpointSubmixDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	if (!EndpointTypeNameSelectorGenerator)
	{
		EndpointTypeNameSelectorGenerator = FNameSelectorGenerator::MakeInstance();
	}

	IDetailCategoryBuilder& EndpointCategory = DetailLayout.EditCategory(TEXT("Endpoint"));

	TSharedPtr<IPropertyHandle> EndpointTypePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEndpointSubmix, EndpointType));
	IDetailPropertyRow& EndpointTypePropertyRow = EndpointCategory.AddProperty(EndpointTypePropertyHandle);

	TWeakPtr<IPropertyHandle> WeakEndpointFormatPtr = EndpointTypePropertyHandle;

	// Set up callbacks for the name selector dropdown menu:
	FNameSelectorGenerator::FNameSelectorCallbacks Callbacks =
	{
		// void OnNameChanged(FName InName)
		[EndpointTypePropertyHandle](FName SelectedName)
		{
			EndpointTypePropertyHandle->SetValue(SelectedName);
		},
		// FName GetCurrentName()
		[EndpointTypePropertyHandle]()
		{
			FName SelectedName;

			EndpointTypePropertyHandle->GetValue(SelectedName);

			return SelectedName;
		},
		// FString GetTooltipText()
		[]()
		{
			return FString(TEXT("Use this to select which endpoint this submix sends to."));
		}
	};

	TArray<FName> AvailableEndpointTypes = IAudioEndpointFactory::GetAvailableEndpointTypes();

	EndpointTypePropertyRow.CustomWidget()
		.NameContent()
		[
			EndpointTypePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(1000.0f)
		.MinDesiredWidth(50.0f)
		[
			EndpointTypeNameSelectorGenerator->MakeNameSelectorWidget(AvailableEndpointTypes, MoveTemp(Callbacks))
		];
}

FSoundfieldEndpointSubmixDetailsCustomization::FSoundfieldEndpointSubmixDetailsCustomization(FProtectedToken ProtectedToken)
	: FNameSelectorGenerator(ProtectedToken)
{
}

TSharedRef<IDetailCustomization> FSoundfieldEndpointSubmixDetailsCustomization::MakeInstance()
{
	TSharedRef<FSoundfieldEndpointSubmixDetailsCustomization> Result = MakeShared<FSoundfieldEndpointSubmixDetailsCustomization>(FProtectedToken{});
	Result->SetWeakThis(StaticCastWeakPtr<FNameSelectorGenerator>(Result.ToWeakPtr()));
	return Result;
}

void FSoundfieldEndpointSubmixDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	if (!EndpointTypeNameSelectorGenerator)
	{
		EndpointTypeNameSelectorGenerator = FNameSelectorGenerator::MakeInstance();
	}

	IDetailCategoryBuilder& EndpointCategory = DetailLayout.EditCategory(TEXT("Endpoint"));

	TSharedPtr<IPropertyHandle> EndpointTypePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USoundfieldEndpointSubmix, SoundfieldEndpointType));
	IDetailPropertyRow& EndpointTypePropertyRow = EndpointCategory.AddProperty(EndpointTypePropertyHandle);

	TWeakPtr<IPropertyHandle> WeakEndpointFormatPtr = EndpointTypePropertyHandle;

	// Set up callbacks for the name selector dropdown menu:
	FNameSelectorGenerator::FNameSelectorCallbacks Callbacks =
	{
		// void OnNameChanged(FName InName)
		[EndpointTypePropertyHandle](FName SelectedName)
		{
			EndpointTypePropertyHandle->SetValue(SelectedName);
		},
		// FName GetCurrentName()
		[EndpointTypePropertyHandle]()
		{
			FName SelectedName;

			EndpointTypePropertyHandle->GetValue(SelectedName);

			return SelectedName;
		},
		// FString GetTooltipText()
		[]()
		{
			return FString(TEXT("Use this to select which endpoint this submix sends to."));
		}
	};

	TArray<FName> AvailableEndpointTypes = ISoundfieldEndpointFactory::GetAllSoundfieldEndpointTypes();

	EndpointTypePropertyRow.CustomWidget()
		.NameContent()
		[
			EndpointTypePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(1000.0f)
		.MinDesiredWidth(50.0f)
		[
			EndpointTypeNameSelectorGenerator->MakeNameSelectorWidget(AvailableEndpointTypes, MoveTemp(Callbacks))
		];
}

#undef LOCTEXT_NAMESPACE



