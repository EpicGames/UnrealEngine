// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinitionDefault_AudioDiffable.generated.h"

UCLASS(Abstract, MinimalAPI)
class UAssetDefinitionDefault_AudioDiffable : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition interface
	AUDIOEDITOR_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	//~ End UAssetDefinition interface

};
