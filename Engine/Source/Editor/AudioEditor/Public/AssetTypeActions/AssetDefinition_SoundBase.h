// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetDefinitionDefault_AudioDiffable.h"
#include "ContentBrowserMenuContexts.h"

#include "AssetDefinition_SoundBase.generated.h"

#define UE_API AUDIOEDITOR_API

// Forward Declarations
struct FToolMenuSection;
class USoundBase;

namespace UE::AudioEditor
{
	AUDIOEDITOR_API void StopSound();
	
	AUDIOEDITOR_API void PlaySound(USoundBase* Sound);
	
	AUDIOEDITOR_API bool IsSoundPlaying(USoundBase* Sound);
	AUDIOEDITOR_API bool IsSoundPlaying(const FAssetData& AssetData);
}

UCLASS(Abstract, MinimalAPI)
class UAssetDefinition_SoundAssetBase : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	// Enables asset types used for audio to customize and register a given menu section
	// for use in the ContentBrowser right-click menu and organize it accordingly.
	UE_API virtual TArray<FToolMenuSection*> RebuildSoundContextMenuSections() const;

	UE_API FToolMenuSection* FindSoundContextMenuSection(FName SectionName) const;
};

UCLASS(MinimalAPI)
class UAssetDefinition_SoundBase : public UAssetDefinition_SoundAssetBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	UE_API virtual FText GetAssetDisplayName() const override;
	UE_API virtual FLinearColor GetAssetColor() const override;
	UE_API virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	UE_API virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& InAssetData) const override;
	UE_API virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const override;
	UE_API virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	UE_API virtual void GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const override;
	// UAssetDefinition End

	UE_API virtual TArray<FToolMenuSection*> RebuildSoundContextMenuSections() const override;

	// Menu Extension statics
	static UE_API void ExecutePlaySound(const FToolMenuContext& InContext);
	static UE_API void ExecuteStopSound(const FToolMenuContext& InContext);
	static UE_API bool CanExecutePlayCommand(const FToolMenuContext& InContext);
	static UE_API ECheckBoxState IsActionCheckedMute(const FToolMenuContext& InContext);
	static UE_API ECheckBoxState IsActionCheckedSolo(const FToolMenuContext& InContext);
	static UE_API void ExecuteMuteSound(const FToolMenuContext& InContext);
	static UE_API void ExecuteSoloSound(const FToolMenuContext& InContext);
	static UE_API void ExecuteClearMutesAndSolos(const FToolMenuContext& InContext);
	static UE_API bool CanExecuteMuteCommand(const FToolMenuContext& InContext);
	static UE_API bool CanExecuteSoloCommand(const FToolMenuContext& InContext);
	static UE_API bool CanExecuteClearMutesAndSolos(const FToolMenuContext& InContext);
	static UE_API bool IsPlayingContextAsset(const FToolMenuContext& InContext, bool bMustMatchContext);
	static UE_API const USoundBase* GetPlayingSound();

	// Asset definition static utilities
	static UE_API TSharedPtr<SWidget> GetSoundBaseThumbnailOverlay(const FAssetData& InAssetData, TFunction<FReply()>&& OnClicked);
	static UE_API void GetSoundBaseAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions);
	static UE_API EAssetCommandResult ActivateSoundBase(const FAssetActivateArgs& ActivateArgs);
};
#undef UE_API
