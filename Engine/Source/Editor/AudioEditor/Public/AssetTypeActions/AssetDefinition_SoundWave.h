// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundWave.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"

#include "AssetDefinition_SoundWave.generated.h"

#define UE_API AUDIOEDITOR_API


UCLASS(MinimalAPI)
class UAssetDefinition_SoundWave : public UAssetDefinition_SoundBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	UE_API virtual FText GetAssetDisplayName() const override;
	UE_API virtual FLinearColor GetAssetColor() const override;
	UE_API virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	UE_API virtual bool CanImport() const;
	// UAssetDefinition End
};
#undef UE_API
