// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetTypeActions_Base.h"

class FAssetTypeActions_AudioDiffable : public FAssetTypeActions_Base
{
public:
	//~ Begin FAssetTypeActions_Base interface
	AUDIOEDITOR_API virtual void PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override;;
	//~ End FAssetTypeActions_Base interface
};
