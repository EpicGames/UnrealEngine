// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_AudioDiffable.h"
#include "AssetTypeCategories.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IAssetTypeActions.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API AUDIOEDITOR_API

class SWidget;
class UClass;
class UObject;
class USoundBase;
struct FAssetData;

class
// UE_DEPRECATED(5.2, "The AssetDefinition system is replacing AssetTypeActions and UAssetDefinition_SoundBase replaced this.  Please see the Conversion Guide in AssetDefinition.h")

FAssetTypeActions_SoundBase : public FAssetTypeActions_AudioDiffable
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundBase", "Sound Base"); }
	virtual FColor GetTypeColor() const override { return FColor(97, 85, 212); }
	UE_API virtual UClass* GetSupportedClass() const override;
	UE_API virtual bool AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Sounds; }
	virtual bool CanFilter() override { return false; }
	UE_API virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;

protected:
	/** Plays the specified sound wave */
	UE_API void PlaySound(USoundBase* Sound) const;

	/** Stops any currently playing sounds */
	UE_API void StopSound() const;

	/** Return true if the specified sound is playing */
	UE_API bool IsSoundPlaying(USoundBase* Sound) const;

	/** Return true if the specified asset's sound is playing */
	UE_API bool IsSoundPlaying(const FAssetData& AssetData) const;

private:
	/** Handler for when PlaySound is selected */
	UE_API void ExecutePlaySound(TArray<TWeakObjectPtr<USoundBase>> Objects) const;

	/** Handler for when StopSound is selected */
	UE_API void ExecuteStopSound(TArray<TWeakObjectPtr<USoundBase>> Objects) const;

	/** Returns true if only one sound is selected to play */
	UE_API bool CanExecutePlayCommand(TArray<TWeakObjectPtr<USoundBase>> Objects) const;

	/** Handler for Mute is selected  */
	UE_API void ExecuteMuteSound(TArray<TWeakObjectPtr<USoundBase>> Objects) const;
	
	/** Handler for Solo is selected  */
	UE_API void ExecuteSoloSound(TArray<TWeakObjectPtr<USoundBase>> Objects) const;

	/** Returns true if the mute state is set.  */
	UE_API bool IsActionCheckedMute(TArray<TWeakObjectPtr<USoundBase>> Objects) const;

	/** Returns true if the solo state is set.  */
	UE_API bool IsActionCheckedSolo(TArray<TWeakObjectPtr<USoundBase>> Objects) const;

	/** Returns true if its possible to mute a sound */
	UE_API bool CanExecuteMuteCommand(TArray<TWeakObjectPtr<USoundBase>> Objects) const;
	
	/** Returns true if its possible to solo a sound */
	UE_API bool CanExecuteSoloCommand(TArray<TWeakObjectPtr<USoundBase>> Objects) const;
};

#undef UE_API
