// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentAssetBroker.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "ISoundClassEditor.h"
#include "ISoundCueEditor.h"
#include "ISoundSubmixEditor.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"

class FAssetEditorToolkit;
class FExtensibilityManager;
class ISoundCueEditor;
class IToolkitHost;
// Forward Declarations
class UDialogueWave;
class UPackage;
class USoundClass;
class USoundCue;
class USoundEffectPreset;
class USoundNode;
class USoundSubmixBase;
class USoundWave;
class UWidgetBlueprint;
struct FDialogueContextMapping;

AUDIOEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioEditor, Log, All);

extern const FName AudioEditorAppIdentifier;

class FAudioComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override;
	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override;
	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override;
};

struct FAudioEditorDeviceSettings
{
	bool bUseSystemDevice = true;
	FString DeviceId;
};

/** Sound class editor module interface */
class IAudioEditorModule :	public IModuleInterface
{
public:
	/** Registers audio editor asset actions. */
	UE_DEPRECATED(5.8, "Asset action where converted to asset definition, there is no need to register them manually anymore")
	virtual void RegisterAssetActions() = 0;

	/** Registers audio editor asset actions specific to audio mixer functionality. */
	UE_DEPRECATED(5.8, "Asset action where converted to asset definition, there is no need to register them manually anymore")
	virtual void RegisterAudioMixerAssetActions() = 0;

	/** Registers effect preset asset actions. */
	virtual void RegisterEffectPresetAssetActions() = 0;

	/** Creates a new sound class editor for a sound class object. */
	virtual TSharedRef<FAssetEditorToolkit> CreateSoundClassEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, USoundClass* InSoundClass ) = 0;

	/** Creates a new sound submix editor for a sound submix object. */
	virtual TSharedRef<FAssetEditorToolkit> CreateSoundSubmixEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, USoundSubmixBase* InSoundSubmix) = 0;

	/** Returns the menu extensibility manager for the given audio editor type. */
	virtual TSharedPtr<FExtensibilityManager> GetSoundClassMenuExtensibilityManager() = 0;

	/** Returns the toolbar extensibility manager for the given audio editor type. */
	virtual TSharedPtr<FExtensibilityManager> GetSoundClassToolBarExtensibilityManager() = 0;

	/** Returns the menu extensibility manager for the given audio editor type. */
	virtual TSharedPtr<FExtensibilityManager> GetSoundSubmixMenuExtensibilityManager() = 0;

	/** Returns the toolbar extensibility manager for the given audio editor type. */
	virtual TSharedPtr<FExtensibilityManager> GetSoundSubmixToolBarExtensibilityManager() = 0;

	/** Creates a new material editor, either for a material or a material function. */
	virtual TSharedRef<ISoundCueEditor> CreateSoundCueEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, USoundCue* SoundCue) = 0;

	/** Returns the menu extensibility manager for the given audio editor type. */
	virtual TSharedPtr<FExtensibilityManager> GetSoundCueMenuExtensibilityManager() = 0;

	/** Returns the toolbar extensibility manager for the given audio editor type. */
	virtual TSharedPtr<FExtensibilityManager> GetSoundCueToolBarExtensibilityManager() = 0;

	/** Replaces sound cue nodes in the graph. */
	virtual void ReplaceSoundNodesInGraph(USoundCue* SoundCue, UDialogueWave* DialogueWave, TArray<USoundNode*>& NodesToReplace, const FDialogueContextMapping& ContextMapping) = 0;

	/** Imports a sound wave from the given package. */
	virtual USoundWave* ImportSoundWave(UPackage* const SoundWavePackage, const FString& InSoundWaveAssetName, const FString& InWavFilename) = 0;

	// Queries if the Audio editor is in "restricted mode" 
	virtual bool IsRestrictedMode() const = 0;

	// Sets if the Audio editor is in "restricted mode" (certain asset actions may be disabled)
	virtual void SetRestrictedMode(bool bInRestricted) = 0;

	// Returns the default audio editor device settings
	virtual TOptional<FAudioEditorDeviceSettings> GetAudioEditorDeviceSettings() const = 0;

	// Allows systems to register to open soundwave files with custom editor
	DECLARE_DELEGATE_OneParam(SoundWaveEditorOpenDelegate, TArray<USoundWave*>);

	SoundWaveEditorOpenDelegate SoundWaveEditorOpen;
};
