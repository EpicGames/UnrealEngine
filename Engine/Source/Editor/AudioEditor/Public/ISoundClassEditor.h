// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

class ISoundClassEditor : public FAssetEditorToolkit
{
public:
	/**
	* Creates a new sound class
	*
	* @param	FromPin		Pin that was dragged to create sound class
	* @param	Location	Location for new sound class
	* @param	Name		Name of the new sound class
	*/
	virtual void CreateSoundClass(class UEdGraphPin* FromPin, const FVector2D& Location, const FString& Name) = 0;
};

