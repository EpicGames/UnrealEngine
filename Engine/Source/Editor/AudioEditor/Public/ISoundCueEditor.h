// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

class USoundCue;
struct Rect;

class ISoundCueEditor : public FAssetEditorToolkit
{
public:
	/** Returns the SoundCue asset inspected by the editor */
	virtual USoundCue* GetSoundCue() const = 0;

	/** Assigns the currently selected nodes to the property control */
	virtual void SetSelection(TArray<UObject*> SelectedObjects) = 0;

	/** Checks whether nodes can currently be pasted */
	virtual bool CanPasteNodes() const = 0;

	/** Paste nodes at a specific location */
	virtual void PasteNodesHere(const FVector2D& Location) = 0;

	/** Get the bounding area for the currently selected nodes
	 *
	 * @param Rect Final output bounding area, including padding
	 * @param Padding An amount of padding to add to all sides of the bounds
	 *
	 * @return false if nothing is selected*/
	virtual bool GetBoundsForSelectedNodes(class FSlateRect& Rect, float Padding) = 0;

	/** Gets the number of nodes that are currently selected */
	virtual int32 GetNumberOfSelectedNodes() const = 0;

	/** Get the currently selected set of nodes */
	virtual TSet<UObject*> GetSelectedNodes() const = 0;
};


