// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"

class UEdGraphPin;

class ISoundSubmixEditor : public FAssetEditorToolkit
{
public:
	/**
	* Creates a new sound submix
	*
	* @param	FromPin		Pin that was dragged to create sound class
	* @param	Location	Location for new sound class
	* @param	Name		Name of the new sound class
	*/
	virtual void CreateSoundSubmix(UEdGraphPin* FromPin, FVector2D Location, const FString& Name) = 0;
};

