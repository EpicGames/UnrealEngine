// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/*-----------------------------------------------------------------------------
   FSoundCueGraphEditorCommands
-----------------------------------------------------------------------------*/

class FSoundClassEditorCommands : public TCommands<FSoundClassEditorCommands>
{
public:
	/** Constructor */
	FSoundClassEditorCommands()
		: TCommands<FSoundClassEditorCommands>("SoundClassEditor", NSLOCTEXT("Contexts", "SoundClassEditor", "SoundClass Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}
	
	/** Plays the SoundCue or stops the currently playing cue/node */
	TSharedPtr<FUICommandInfo> ToggleSolo;

	/** Plays the SoundCue or stops the currently playing cue/node */
	TSharedPtr<FUICommandInfo> ToggleMute;

	/** Shows the reference viewer for the selected assets */
	TSharedPtr<FUICommandInfo> ViewReferences;

	/** Show the Find in Sound Class Graph window. */
	TSharedPtr<FUICommandInfo> FindInSoundClassGraph;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
