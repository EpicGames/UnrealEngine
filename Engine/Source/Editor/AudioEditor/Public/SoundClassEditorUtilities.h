// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/Vector2D.h"

#define UE_API AUDIOEDITOR_API


class FSoundClassEditorUtilities
{
public:

	/**
	 * Create a new SoundClass
	 *
	 * @param	Graph		Graph representing sound classes
	 * @param	FromPin		Pin that was dragged to create sound class
	 * @param	Location	Location for new sound class
	 */
	static UE_API void CreateSoundClass(const class UEdGraph* Graph, class UEdGraphPin* FromPin, const FVector2D& Location, FString Name);

private:
	FSoundClassEditorUtilities() {}
};

#undef UE_API
