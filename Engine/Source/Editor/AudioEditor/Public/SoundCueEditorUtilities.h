// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

#define UE_API AUDIOEDITOR_API

class ISoundCueEditor;
class UEdGraph;
class UObject;
class USoundCue;
struct Rect;

//////////////////////////////////////////////////////////////////////////
// FSoundCueEditorUtilities

class FSoundCueEditorUtilities
{
public:

	/** Can we paste to this graph? */
	static UE_API bool CanPasteNodes(const class UEdGraph* Graph);

	/** Perform paste on graph, at location */
	static UE_API void  PasteNodesHere(class UEdGraph* Graph, const FVector2D& Location);

	/** Adds SoundNodeWavePlayer nodes based on selected objects
	 *
	 * @param SelectedWaves List of selected SoundWaves to create player nodes for
	 * @param SoundCue The SoundCue that the nodes will be part of
	 * @param OutPlayers Stores all created nodes
	 * @param Location Position of first created node
	 */
	static UE_API void CreateWaveContainers(TArray<class USoundWave*>& SelectedWaves, class USoundCue* SoundCue, TArray<class USoundNode*>& OutPlayers, FVector2D Location);

	/** Adds USoundNodeDialoguePlayer nodes based on selected objects
	 *
	 * @param SelectedDialogues List of selected DialogueWaves to create player nodes for
	 * @param SoundCue The SoundCue that the nodes will be part of
	 * @param OutPlayers Stores all created nodes
	 * @param Location Position of first created node
	 */
	static UE_API void CreateDialogueContainers(TArray<class UDialogueWave*>& SelectedDialogues, USoundCue* SoundCue, TArray<class USoundNode*>& OutPlayers, FVector2D Location);


	/** Get the bounding area for the currently selected nodes
	 *
	 * @param Graph The Graph we are finding bounds for
	 * @param Rect Final output bounding area, including padding
	 * @param Padding An amount of padding to add to all sides of the bounds
	 *
	 * @return false if nothing is selected*/
	static UE_API bool GetBoundsForSelectedNodes(const UEdGraph* Graph, class FSlateRect& Rect, float Padding = 0.0f);

	/** Gets the number of nodes that are currently selected */
	static UE_API int32 GetNumberOfSelectedNodes(const UEdGraph* Graph);

	/** Get the currently selected set of nodes */
	static UE_API TSet<UObject*> GetSelectedNodes(const UEdGraph* Graph);

private:
	/** Get ISoundCueEditor for given object, if it exists */
	static UE_API TSharedPtr<class ISoundCueEditor> GetISoundCueEditorForObject(const UObject* ObjectToFocusOn);

	FSoundCueEditorUtilities() {}
};

#undef UE_API
