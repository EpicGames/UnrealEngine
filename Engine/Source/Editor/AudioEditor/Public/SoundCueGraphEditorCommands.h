// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/*-----------------------------------------------------------------------------
   FSoundCueGraphEditorCommands
-----------------------------------------------------------------------------*/

class FSoundCueGraphEditorCommands : public TCommands<FSoundCueGraphEditorCommands>
{
public:
	/** Constructor */
	FSoundCueGraphEditorCommands() 
		: TCommands<FSoundCueGraphEditorCommands>("SoundCueGraphEditor", NSLOCTEXT("Contexts", "SoundCueGraphEditor", "SoundCue Graph Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}
	
	/** Plays the SoundCue */
	TSharedPtr<FUICommandInfo> PlayCue;
	
	/** Stops the currently playing cue/node */
	TSharedPtr<FUICommandInfo> StopCueNode;

	/** Plays the selected node */
	TSharedPtr<FUICommandInfo> PlayNode;

	/** Plays the SoundCue or stops the currently playing cue/node */
	TSharedPtr<FUICommandInfo> TogglePlayback;

	/** Selects the SoundWave in the content browser */
	TSharedPtr<FUICommandInfo> BrowserSync;

	/** Breaks the node input/output link */
	TSharedPtr<FUICommandInfo> BreakLink;

	/** Adds an input to the node */
	TSharedPtr<FUICommandInfo> AddInput;

	/** Removes an input from the node */
	TSharedPtr<FUICommandInfo> DeleteInput;

	/** Plays the SoundCue or stops the currently playing cue/node */
	TSharedPtr<FUICommandInfo> ToggleSolo;

	/** Plays the SoundCue or stops the currently playing cue/node */
	TSharedPtr<FUICommandInfo> ToggleMute;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
