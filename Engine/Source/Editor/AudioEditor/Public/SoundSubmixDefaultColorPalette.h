// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Sound/SoundSubmix.h"
#include "UObject/NameTypes.h"

class USoundSubmixBase;

namespace Audio
{
	// Use the following values for the Mauve theme:
	/*
	const FColor DefaultSubmixColor = FColor(122, 183, 27);
	const FColor SoundfieldDefaultSubmixColor = FColor(223, 140, 132);
	const FColor EndpointDefaultSubmixColor = FColor(213, 191, 106);
	const FColor SoundfieldEndpointDefaultSubmixColor = FColor(208, 236, 255);
	*/

	const FColor DefaultSubmixColor = FColor(143, 190, 0);
	const FColor SoundfieldDefaultSubmixColor = FColor(0, 168, 198);
	const FColor EndpointDefaultSubmixColor = FColor(249, 242, 231);
	const FColor SoundfieldEndpointDefaultSubmixColor = FColor(64, 192, 203);

	// These names are used to identify pins by their owning node.
	const FName SoundSubmixName = TEXT("SoundSubmix");
	const FName SoundfieldSubmixName = TEXT("SoundfieldSubmix");
	const FName EndpointSubmixName = TEXT("EndpointSubmix");
	const FName SoundfieldEndpointSubmixName = TEXT("SoundfieldEndpointSubmix");

	// These utility functions are used
	FName GetNameForSubmixType(const USoundSubmixBase* InSubmix);
	FColor GetColorForSubmixType(const USoundSubmixBase* InSubmix);
	FColor GetColorForSubmixType(const FName& InSubmixName);
	const bool IsConnectionPerformingSoundfieldConversion(const USoundSubmixBase* InputSubmix, const USoundSubmixBase* OutputSubmix);
}