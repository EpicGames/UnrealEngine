// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/Vector2D.h"

#define UE_API AUDIOEDITOR_API

class UEdGraph;
class UEdGraphPin;

class FSoundSubmixEditorUtilities
{
public:

	/**
	 * Create a new SoundSubmix
	 *
	 * @param	Graph		Graph representing sound classes
	 * @param	FromPin		Pin that was dragged to create sound class
	 * @param	Location	Location for new sound class
	 */
	static UE_API void CreateSoundSubmix(const UEdGraph* Graph, UEdGraphPin* FromPin, FVector2D Location, const FString& Name);

private:
	FSoundSubmixEditorUtilities() {}
};

#undef UE_API
