// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "PropertyHandle.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"

#define UE_API AUDIOEDITOR_API

class IDetailGroup;
class IDetailLayoutBuilder;
class SWidget;

// Utility class to build combo boxes out of arrays of names.
class FNameSelectorGenerator
{
protected:
	struct FProtectedToken { explicit FProtectedToken() = default; };

public:
	FNameSelectorGenerator(FProtectedToken);

	struct FNameSelectorCallbacks
	{
		TUniqueFunction<void(FName)> OnNewNameSelected;
		TUniqueFunction<FName()> GetCurrentlySelectedName;
		TUniqueFunction<FString()> GetTooltipText;
	};

	// Use this to generate a combo box widget.
	TSharedRef<SWidget> MakeNameSelectorWidget(TArray<FName>& InNameArray, FNameSelectorCallbacks&& InCallbacks);

	// Makes a new instance of the name selector generator class
	static TSharedRef<FNameSelectorGenerator> MakeInstance();

protected:
	// This needs to be called after construction and after it has been bound to a TSharedPtr
	void SetWeakThis(TWeakPtr<FNameSelectorGenerator>&& InWeakThis);

	void OnSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> HandleResponseComboBoxGenerateWidget(TSharedPtr<FName> StringItem);
	FText GetComboBoxToolTip() const;
	FText GetComboBoxContent() const;

	TArray<TSharedPtr<FName>> CachedNameArray;
	FNameSelectorCallbacks CachedCallbacks;

private:
	TWeakPtr<FNameSelectorGenerator> WeakThis;
};

class FSoundfieldSubmixDetailsCustomization : public IDetailCustomization
{
protected:
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	UE_API FSoundfieldSubmixDetailsCustomization(FPrivateToken);

	// Makes a new instance of this detail layout class
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	TSharedPtr<FNameSelectorGenerator> SoundfieldFormatNameSelectorGenerator;
};

class FEndpointSubmixDetailsCustomization : public IDetailCustomization, public FNameSelectorGenerator
{
public:
	UE_API FEndpointSubmixDetailsCustomization(FProtectedToken);

	// Makes a new instance of this detail layout class
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	TSharedPtr<FNameSelectorGenerator> EndpointTypeNameSelectorGenerator;
};

class FSoundfieldEndpointSubmixDetailsCustomization : public IDetailCustomization, public FNameSelectorGenerator
{
public:
	UE_API FSoundfieldEndpointSubmixDetailsCustomization(FProtectedToken);

	// Makes a new instance of this detail layout class
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	TSharedPtr<FNameSelectorGenerator> EndpointTypeNameSelectorGenerator;
};

#undef UE_API
